[CoreRedirects]
; Blueprints
+ClassRedirects=(OldName="/Script/PointCloudRuntime.PointCloud",NewName="LidarPointCloud")
+ClassRedirects=(OldName="/Script/PointCloudRuntime.PointCloudPluginComponent",NewName="LidarPointCloudComponent")
+ClassRedirects=(OldName="/Script/PointCloudRuntime.PointCloudActor",NewName="LidarPointCloudActor")
+ClassRedirects=(OldName="/Script/PointCloudRuntime.PointCloudBlueprintLibrary",NewName="LidarPointCloudBlueprintLibrary")
+StructRedirects=(OldName="/Script/PointCloudRuntime.PointCloudPoint",NewName="LidarPointCloudPoint")
+StructRedirects=(OldName="/Script/PointCloudRuntime.PointCloudTraceHit",NewName="LidarPointCloudTraceHit")
+StructRedirects=(OldName="/Script/PointCloudRuntime.PointCloudImportSettings_ASCII_Columns",NewName="LidarPointCloudImportSettings_ASCII_Columns")
+EnumRedirects=(OldName="/Script/PointCloudRuntime.EPointCloudColorationMode",NewName="ELidarPointCloudColorationMode")
+EnumRedirects=(OldName="/Script/PointCloudRuntime.EPointCloudSpriteShape",NewName="ELidarPointCloudSpriteShape")
+EnumRedirects=(OldName="/Script/PointCloudRuntime.EPointCloudDuplicateHandling",NewName="ELidarPointCloudDuplicateHandling")
+EnumRedirects=(OldName="/Script/PointCloudRuntime.EPointCloudAsyncMode",NewName="ELidarPointCloudAsyncMode")
+PropertyRedirects=(OldName="/Script/LidarPointCloudRuntime.LidarPointCloud.CollisionAccuracy",NewName="MaxCollisionError")

; Materials
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/M_PointCloud",NewName="/LidarPointCloud/Materials/M_LidarPointCloud")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/M_PointCloud.M_PointCloud",NewName="/LidarPointCloud/Materials/M_LidarPointCloud.M_LidarPointCloud")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/M_PointCloud_Masked",NewName="/LidarPointCloud/Materials/M_LidarPointCloud_Masked")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/M_PointCloud_Masked.M_PointCloud_Masked",NewName="/LidarPointCloud/Materials/M_LidarPointCloud_Masked.M_LidarPointCloud_Masked")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/M_PP_EDL_MainPass",NewName="/LidarPointCloud/Materials/M_PP_EDL_MainPass")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/M_PP_EDL_MainPass.M_PP_EDL_MainPass",NewName="/LidarPointCloud/Materials/M_PP_EDL_MainPass.M_PP_EDL_MainPass")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/MF_EDF_SampleTexel_MainPass",NewName="/LidarPointCloud/Materials/MF_EDF_SampleTexel_MainPass")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/MF_EDF_SampleTexel_MainPass.MF_EDF_SampleTexel_MainPass",NewName="/LidarPointCloud/Materials/MF_EDF_SampleTexel_MainPass.MF_EDF_SampleTexel_MainPass")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/MF_EDF_SampleTexel_CustomPass",NewName="/LidarPointCloud/Materials/MF_EDF_SampleTexel_CustomPass")
+PackageRedirects=(OldName="/PointCloudPlugin/Materials/MF_EDF_SampleTexel_CustomPass.MF_EDF_SampleTexel_CustomPass",NewName="/LidarPointCloud/Materials/MF_EDF_SampleTexel_CustomPass.MF_EDF_SampleTexel_CustomPass")

; Package
+PackageRedirects=(MatchWildcard=true,OldName=".../PointCloudRuntime...",NewName="/LidarPointCloudRuntime")