{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiDAR Point Cloud Support",
	"Description": "Adds support for importing, processing and rendering of LiDAR Point Clouds.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Mac",
		"Linux"
	],
	"Modules": [
		{
			"Name": "LidarPointCloudRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		},
		{
			"Name": "LidarPointCloudEditor",
			"Type": "Editor",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	]
}