// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/VertexFactoryCommon.ush"

/*=============================================================================
	LidarPointCloudVertexFactory.usf: LiDAR Point Cloud vertex factory shader code.
=============================================================================*/

// First half is normal, second half has inverted V
static float2 PrecomputedUV_Quad[8] = { float2(-0.5, -0.5), float2(0.5, -0.5), float2(0.5, 0.5), float2(-0.5, 0.5), float2(0, 1), float2(1, 1), float2(1, 0), float2(0, 0) };

// Used to detect if the instance is rendered inside the editor viewport
uint bEditorView;

// Selection
half3 SelectionColor;

// Alignment
float3 LocationOffset;

// Used for sprite size calculation
float VirtualDepth;
float SpriteSizeMultiplier;
float RootCellSize;
float3 RootExtent;
uint bUsePerPointScaling;
float ReversedVirtualDepthMultiplier;
#if !RAYHITGROUPSHADER
Buffer<float> TreeBuffer;
#endif

// Needed for WPO calculations
float3 ViewRightVector;
float3 ViewUpVector;
uint bUseCameraFacing;
uint bUseScreenSizeScaling;
uint bUseStaticBuffers;

// Used for coloration override
float3 BoundsSize;
half3 ElevationColorBottom;
half3 ElevationColorTop;

// Color Adjustment
half4 Offset;
half4 Contrast;
half4 Saturation;
half4 Gamma;
half3 Tint;
float IntensityInfluence;

// Flags
uint bUseCircle;
uint bUseColorOverride;
uint bUseElevationColor;
uint bUseClassification;
uint bUseClassificationAlpha;

float4 ClassificationColors[32];

/**
 * [Tx, Ty, Tz, Invert]
 * [Fx, Fy, Fz, Ex]
 * [Rx, Ry, Rz, Ey]
 * [Ux, Uy, Uz, Ez]
 */
float4x4 ClippingVolume[16];
uint NumClippingVolumes;
uint bStartClipped;

struct FVertexFactoryInput
{
    float4	Position    : ATTRIBUTE0;
    half4	Color		: ATTRIBUTE1;
    uint	MetaData	: ATTRIBUTE2;
	uint    VertexId    : SV_VertexID;

	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

struct FPositionOnlyVertexFactoryInput
{
	float4  Position    : ATTRIBUTE0;
	uint    VertexId    : SV_VertexID;

	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

struct FPositionAndNormalOnlyVertexFactoryInput
{
	float4  Position	: ATTRIBUTE0;
	float4	Normal		: ATTRIBUTE2;
	uint    VertexId	: SV_VertexID;

	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

	half4	Color : COLOR0;
	half	Alpha : COLOR1;
	float2	TexCoords : TEXCOORD0;

};

struct FVertexFactoryIntermediates
{
    half3x3 TangentToLocal;
    half3x3 TangentToWorld;
    half TangentToWorldSign;
    half4 Color;
	half Alpha;
	float3 Position;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

float3 ApplyClipping(float3 Position)
{
	bool bClip = bStartClipped;

	// Only process clipping if not in the editor view
	if (!bEditorView)
	{
		for (uint i = 0; i < NumClippingVolumes; i++)
		{
			float3 DeltaPosition = DFDemote(TransformLocalToWorld(Position)).xyz - ClippingVolume[i][0].xyz;
			bool bInsideClippingBox = abs(dot(DeltaPosition, ClippingVolume[i][1].xyz)) <= ClippingVolume[i][1].w && abs(dot(DeltaPosition, ClippingVolume[i][2].xyz)) <= ClippingVolume[i][2].w && abs(dot(DeltaPosition, ClippingVolume[i][3].xyz)) <= ClippingVolume[i][3].w;

			if (ClippingVolume[i][0][3] == 1)
			{
				if (bInsideClippingBox)
				{
					bClip = true;
				}
			}
			else
			{
				if (bInsideClippingBox)
				{
					bClip = false;
				}
			}
		}
	}

	return (!bEditorView && bClip) ? 0.0f / 0 : Position;
}

half3x3 CalcTangentToLocal(uint MetaData)
{
	half3x3 TangentToLocal;

	if (bUseCameraFacing)
	{
		// In this mode, [0] and [1] are only really used for scaling
		TangentToLocal[0] = ViewRightVector;
		TangentToLocal[1] = ViewUpVector;

		// Hardcoded to have the lighting behave more correctly
		TangentToLocal[2] = float3(0, 0, 1);
	}
	else
	{
		half3 Normal = half3((0x000000FF & MetaData) / 127.5f - 1, (0x000000FF & (MetaData >> 8)) / 127.5f - 1, (0x000000FF & (MetaData >> 16)) / 127.5f - 1);

		// Force camera facing
		if (dot(Normal, Normal) < 0.1f)
		{
			// In this mode, [0] and [1] are only really used for scaling
			TangentToLocal[0] = ViewRightVector;
			TangentToLocal[1] = ViewUpVector;

			// Hardcoded to have the lighting behave more correctly
			TangentToLocal[2] = float3(0, 0, 1);
		}
		else
		{
			half3 N = abs(Normal);

			half3 Tangent = half3(1, 0, 0);

			// Find best basis vectors.
			if (N.z > N.x && N.z > N.y)
			{
				Tangent = half3(1, 0, 0);
			}
			else
			{
				Tangent = half3(0, 0, 1);
			}

			Tangent = (Tangent - Normal * dot(Tangent, Normal));
			Tangent = Tangent * rsqrt(dot(Tangent, Tangent));

			TangentToLocal[0] = Tangent;
			TangentToLocal[1] = cross(Tangent, Normal);
			TangentToLocal[2] = Normal;
		}
	}

	return TangentToLocal;
}

half3x3 CalcTangentToWorldNoScale(in half3x3 TangentToLocal)
{
	half3x3 LocalToWorld = GetLocalToWorld3x3();
	half3 InvScale = Primitive.InvNonUniformScale;
	LocalToWorld[0] *= InvScale.x;
	LocalToWorld[1] *= InvScale.y;
	LocalToWorld[2] *= InvScale.z;
	return mul(TangentToLocal, LocalToWorld);
}

float CalcSpriteSize(float3 Position)
{
	float VD = VirtualDepth;

#if !RAYHITGROUPSHADER
	if (bUsePerPointScaling)
	{
		VD = 0;
		uint Idx = 0;
		float3 NodeExtent = RootExtent;
		float3 NodeCenter = LocationOffset;

		while (true)
		{
			uint TreeBufferData = asuint(TreeBuffer[Idx]);
			uint ChildrenBitmask = 0x000000FF & TreeBufferData;

			float3 CenterRelativeLocation = Position - NodeCenter;
			uint ChildNodeIndex = (CenterRelativeLocation.x > 0 ? 4 : 0) + (CenterRelativeLocation.y > 0 ? 2 : 0) + (CenterRelativeLocation.z > 0);

			if ((ChildrenBitmask & (1u << ChildNodeIndex)) == 0)
			{
				break;
			}

			Idx += (0x0000FFFF & (TreeBufferData >> 8));
			for (uint i = 0; i < 8; i++)
			{
				if (i == ChildNodeIndex)
				{
					break;
				}

				if ((ChildrenBitmask & (1u << i)) != 0)
				{
					Idx++;
				}
			}

			VD = (0x000000FF & (TreeBufferData >> 24)) * ReversedVirtualDepthMultiplier;
			NodeCenter += NodeExtent * (float3((ChildNodeIndex & 4) == 4, (ChildNodeIndex & 2) == 2, (ChildNodeIndex & 1) == 1) - 0.5f);
			NodeExtent *= 0.5f;
		}
	}
#endif
	
	return RootCellSize / pow(2.0f, VD);
}

float3 ProcessPosition(float3 Position, half3x3 TangentToLocal, int VertexID)
{
	const float Size = bUseScreenSizeScaling ? mul(float4(Position.xyz, 1), ResolvedView.TranslatedWorldToView).z * 0.01f : CalcSpriteSize(Position);
	const half2 Scale = PrecomputedUV_Quad[VertexID % 4];
	return Position + (TangentToLocal[0] * Scale.x + TangentToLocal[1] * Scale.y) * SpriteSizeMultiplier * Size;
}

float3 ProcessPosition(float3 Position, int VertexID)
{
	half3x3 TangentToLocal;
	TangentToLocal[0] = ViewRightVector;
	TangentToLocal[1] = ViewUpVector;
	return ProcessPosition(Position, TangentToLocal, VertexID);
}

float3 LidarGetInstancePosition(uint VertexId)
{
	uint Idx = (VertexId / 4) * 5;
	return float3(LidarVF.VertexFetch_Buffer[Idx], LidarVF.VertexFetch_Buffer[Idx + 1], LidarVF.VertexFetch_Buffer[Idx + 2]);
}

float3 LidarGetInstanceNormal(uint VertexId)
{
	uint Idx = (VertexId / 4) * 5;
	return CalcTangentToLocal(asuint(LidarVF.VertexFetch_Buffer[Idx + 4]))[2];
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);

	float3 Position;
	half3 Color;
	half Intensity;
	uint MetaData;
	
	if (bUseStaticBuffers)
	{
		Position = Input.Position.xyz;
		MetaData = Input.MetaData;
		Color = Input.Color.rgb;
		Intensity = Input.Color.a;
	}
	else
	{
		uint Idx = (Input.VertexId / 4) * 5;
		Position = float3(LidarVF.VertexFetch_Buffer[Idx], LidarVF.VertexFetch_Buffer[Idx + 1], LidarVF.VertexFetch_Buffer[Idx + 2]);
		
		uint ColorData = asuint(LidarVF.VertexFetch_Buffer[Idx + 3]);		
		Color = half3(0x000000FF & (ColorData >> 16), 0x000000FF & (ColorData >> 8), 0x000000FF & ColorData) * 0.003921568627451;
		Intensity = (0x000000FF & (ColorData >> 24)) * 0.003921568627451;

		MetaData = asuint(LidarVF.VertexFetch_Buffer[Idx + 4]);
	}

	Intermediates.TangentToLocal = CalcTangentToLocal(MetaData);
	Intermediates.TangentToWorldSign = Intermediates.SceneData.InstanceData.DeterminantSign;
	Intermediates.TangentToWorld = CalcTangentToWorldNoScale(Intermediates.TangentToLocal);
	Intermediates.Position = ApplyClipping(Position + LocationOffset);
	
	// Color
	{
		MetaData = 0x000000FF & (MetaData >> 24);
		uint bSelected = clamp(0x00000040 & MetaData, 0, 1);
		uint Classification = 0x0000001F & (MetaData >> 1);

		half3 ElevationColor = lerp(ElevationColorBottom, ElevationColorTop, clamp((Position.z + BoundsSize.z * 0.5) / BoundsSize.z, 0, 1));
		half3 PositionColor = (Position + BoundsSize * 0.5) / BoundsSize;
		float4 ClassificationColor = ClassificationColors[Classification];

		// Pick color source
		Color = lerp(Color, lerp(lerp(PositionColor, ClassificationColor.rgb, bUseClassification), ElevationColor, bUseElevationColor), bUseColorOverride);

		// Mix with Intensity
		Color = lerp(Color, Color * Intensity, IntensityInfluence);

		// Color Adjustment
		{
			Color = lerp(dot(Color, half3(0.299, 0.587, 0.144)), Color, Saturation.rgb * Saturation.a);
			Color *= Tint;
			Color = pow(Color, 2.2 * Gamma.rgb * Gamma.a);
			Color = (Color - 0.5) * Contrast.rgb * Contrast.a + 0.5;
			Color += Offset.rgb * Offset.a;
			Color = clamp(Color, 0, 1);
		}

		// Selection
		Color = lerp(Color, SelectionColor, bSelected);

		Intermediates.Color = half4(Color, bUseCircle);
		Intermediates.Alpha = lerp(1, ClassificationColor.a, bUseClassification || bUseClassificationAlpha);
	}

	return Intermediates;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants = (FVertexFactoryInterpolantsVSToPS)0;
	Interpolants.TangentToWorld0 = float4(Intermediates.TangentToWorld[0], 0);
	Interpolants.TangentToWorld2 = float4(Intermediates.TangentToWorld[2], Intermediates.TangentToWorldSign);
	Interpolants.Color = VertexParameters.VertexColor;
	Interpolants.Alpha = Intermediates.Alpha;
	Interpolants.TexCoords = PrecomputedUV_Quad[Input.VertexId % 4 + 4];    // Using inverted V

	return Interpolants;
}

half3x3 VertexFactoryGetTangentToLocal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToLocal;
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float4 WorldPosition = TransformLocalToTranslatedWorld(Intermediates.Position, Intermediates.SceneData.Primitive.LocalToWorld);
	WorldPosition.xyz = ProcessPosition(WorldPosition.xyz, Intermediates.TangentToLocal, Input.VertexId);
	return WorldPosition;
}
// local position relative to instance
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	// No support for instancing, so instance == primitive
	return Intermediates.Position;
}
float4 VertexFactoryGetWorldPosition(FPositionOnlyVertexFactoryInput Input)
{
	float4 WorldPosition = TransformLocalToTranslatedWorld(
		ApplyClipping((bUseStaticBuffers ? Input.Position.xyz : LidarGetInstancePosition(Input.VertexId)) + LocationOffset),
		GetPrimitiveDataFromUniformBuffer().LocalToWorld);
	WorldPosition.xyz = ProcessPosition(WorldPosition.xyz, Input.VertexId);
	return WorldPosition;
}
float4 VertexFactoryGetWorldPosition(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	float4 WorldPosition = TransformLocalToTranslatedWorld(
		ApplyClipping((bUseStaticBuffers ? Input.Position.xyz : LidarGetInstancePosition(Input.VertexId)) + LocationOffset),
		GetPrimitiveDataFromUniformBuffer().LocalToWorld);
	WorldPosition.xyz = ProcessPosition(WorldPosition.xyz, Input.VertexId);
	return WorldPosition;
}
float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}
float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}
float3 VertexFactoryGetWorldNormal(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	return RotateLocalToWorld(bUseStaticBuffers ? Input.Normal.xyz : LidarGetInstanceNormal(Input.VertexId));
}
float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToWorld[2];
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
    return mul(float4(Intermediates.Position, 1), DFMultiplyAffineTranslationDemote(Intermediates.SceneData.InstanceData.PrevLocalToWorld, ResolvedView.PrevPreViewTranslation));
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	// No support for instancing, so instance == primitive
	return Intermediates.Position;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
    FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
    Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

    Result.TangentToWorld = Intermediates.TangentToWorld;
    Result.PreSkinnedNormal = TangentToLocal[2];
    Result.PreSkinnedPosition = WorldPosition;
    Result.VertexColor = Intermediates.Color;

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	Result.TexCoords[0] = PrecomputedUV_Quad[Input.VertexId % 4];
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

    return Result;
}

FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
    FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

    Result.TwoSidedSign = 1;
	Result.VertexColor = Interpolants.Color;
	Result.VertexColor.a = saturate((lerp(1, 1 - distance(Interpolants.TexCoords, 0.5), Interpolants.Color.a) - 0.5) * 10000) * Interpolants.Alpha;

    half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
    half4 TangentToWorld2 = Interpolants.TangentToWorld2;
    Result.UnMirrored = TangentToWorld2.w;
    Result.TangentToWorld = AssembleTangentToWorld(TangentToWorld0, TangentToWorld2);

#if NUM_TEX_COORD_INTERPOLATORS
    Result.TexCoords[0] = Interpolants.TexCoords;
#endif

    return Result;
}

#if USE_INSTANCING
#define VF_IMPLEMENTED_GET_INSTANCE_HIT_PROXY_ID

float4 VertexFactoryGetInstanceHitProxyId(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates) { return 0; }
#endif

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveDataFromUniformBuffer();
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	struct FVertexFactoryRayTracingInterpolants
	{
		FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
	};

	float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
	{
#if NUM_MATERIAL_TEXCOORDS
		return Interpolants.InterpolantsVSToPS.TexCoords.xy;
#else // #if NUM_MATERIAL_TEXCOORDS
		return float2(0,0);
#endif // #if NUM_MATERIAL_TEXCOORDS
	}

	FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
	{
		return Input.InterpolantsVSToPS;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
	{
		FVertexFactoryRayTracingInterpolants Interpolants;
		Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
		return Interpolants;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
	{
		FVertexFactoryRayTracingInterpolants O;
	
		// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
		// interpolated normal from 'whatever' interpolation scheme we're using
	
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);

#if INTERPOLATE_VERTEX_COLOR
		INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords);
#endif

		return O;
	}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#if RAYHITGROUPSHADER || COMPUTESHADER
uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return 0;
}
#endif

FVertexFactoryInput LoadVertexFactoryInputFromIndices(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

	Input.VertexId = TriangleIndex * 3 + VertexIndex;

	/**
	 * When using static buffers we duplicate the data for each of the 4 vertices of the plane so we can use the VertexId directly
	 * When using dynamic buffers, each 4 vertices of the plane will share the same data element so we need to divide by 4
	 *
	 * The buffer stride is 20 bytes and data is held in floats, so we need to multiply the starting offset by 5
	 */
	uint VertexOffset = (bUseStaticBuffers ? Input.VertexId : floor(Input.VertexId / 4)) * 5;

	// Helps minimize incorrect self-shadowing due to camera-facing planes 
	float ZOffset = 0.3;
		
	Input.Position.x = LidarVF.VertexFetch_Buffer[VertexOffset + 0];
	Input.Position.y = LidarVF.VertexFetch_Buffer[VertexOffset + 1];
	Input.Position.z = LidarVF.VertexFetch_Buffer[VertexOffset + 2] - ZOffset;

	uint ColorData = asuint(LidarVF.VertexFetch_Buffer[VertexOffset + 3]);
	Input.Color = half4(0x000000FF & (ColorData >> 16), 0x000000FF & (ColorData >> 8), 0x000000FF & ColorData, 0x000000FF & (ColorData >> 24)) * 0.003921568627451;
	Input.MetaData = asuint(LidarVF.VertexFetch_Buffer[VertexOffset + 4]);

	return Input;
}

#if RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input = LoadVertexFactoryInputFromIndices(TriangleIndex, VertexIndex);
		
#if VF_USE_PRIMITIVE_SCENE_DATA
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);
#endif // VF_USE_PRIMITIVE_SCENE_DATA
		
	return Input;
}
#endif

#if COMPUTESHADER

FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input = LoadVertexFactoryInputFromIndices(TriangleIndex, VertexIndex);
		
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, PrimitiveData.InstanceSceneDataOffset + DrawInstanceId, DrawInstanceId);

	return Input;
}

#endif

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"

