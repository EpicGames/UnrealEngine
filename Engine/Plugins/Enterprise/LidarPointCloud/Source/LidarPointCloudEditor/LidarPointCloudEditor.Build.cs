// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class LidarPointCloudEditor : ModuleRules
    {
        public LidarPointCloudEditor(ReadOnlyTargetRules Target) : base(Target)
        {
            PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                "AssetDefinition",
                "CoreUObject",
                "ContentBrowser",
				"ToolMenus",
                "Engine",
                "Slate",
                "SlateCore",
				"EditorFramework",
                "UnrealEd",
                "KismetWidgets",
                "PropertyEditor",
                "ApplicationCore",
                "InputCore",
                "MainFrame",
                "LidarPointCloudRuntime",
                "Projects",
                "UMG",
                "Core",
                "CoreUObject",
                "Engine",
                "RenderCore",
                "RHI",
                "AdvancedPreviewScene",
                "EditorInteractiveToolsFramework",
                "InteractiveToolsFramework",
                "StatusBar",
                "MeshDescription",
                "StaticMeshDescription"
                }
            );
        }
    }
}