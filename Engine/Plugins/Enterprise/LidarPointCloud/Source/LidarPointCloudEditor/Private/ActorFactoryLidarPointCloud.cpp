// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryLidarPointCloud.h"
#include "LidarPointCloudActor.h"
#include "LidarPointCloudComponent.h"
#include "LidarPointCloud.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "ActorFactory"

UActorFactoryLidarPointCloud::UActorFactoryLidarPointCloud(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("LidarPointCloudDisplayName", "LiDAR Point Cloud");
	NewActorClass = ALidarPointCloudActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryLidarPointCloud::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(ULidarPointCloud::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoPointCloud", "A valid LiDAR Point Cloud must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryLidarPointCloud::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	ULidarPointCloud* PointCloud = CastChecked<ULidarPointCloud>(Asset);

	UE_LOGF(LogActorFactory, Log, "Actor Factory created %ls", *PointCloud->GetName());

	// Change properties
	ALidarPointCloudActor* PointCloudActor = CastChecked<ALidarPointCloudActor>(NewActor);
	PointCloudActor->GetPointCloudComponent()->SetPointCloud(PointCloud);
}

UObject* UActorFactoryLidarPointCloud::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	ALidarPointCloudActor* PCA = CastChecked<ALidarPointCloudActor>(Instance);

	return PCA->GetPointCloudComponent()->GetPointCloud();
}

FQuat UActorFactoryLidarPointCloud::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	// Meshes align the Z (up) axis with the surface normal
	return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}

#undef LOCTEXT_NAMESPACE