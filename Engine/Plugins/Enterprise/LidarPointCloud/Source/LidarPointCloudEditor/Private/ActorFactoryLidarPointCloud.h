// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryLidarPointCloud.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI, config = Editor)
class UActorFactoryLidarPointCloud : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	virtual FQuat AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const override;
	// End UActorFactory Interface
};