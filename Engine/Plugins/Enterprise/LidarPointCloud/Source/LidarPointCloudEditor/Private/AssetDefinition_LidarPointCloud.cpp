// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_LidarPointCloud.h"

#include "ContentBrowserMenuContexts.h"
#include "PackageTools.h"
#include "ToolMenus.h"

#include "LidarPointCloudEditor.h"
#include "LidarPointCloudEditorHelper.h"

#include "LidarPointCloud.h"

#define LOCTEXT_NAMESPACE "LidarPointCloud"


FText UAssetDefinition_LidarPointCloud::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_LidarPointCloud", "LiDAR Point Cloud");
}

EAssetCommandResult UAssetDefinition_LidarPointCloud::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (ULidarPointCloud* PointCloud : OpenArgs.LoadObjects<ULidarPointCloud>())
	{
		TSharedRef<FLidarPointCloudEditor> NewPointCloudEditor(new FLidarPointCloudEditor());
		NewPointCloudEditor->InitPointCloudEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, PointCloud);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_LidarPointCloud::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Misc };
	return Categories;
}

TSoftClassPtr<UObject> UAssetDefinition_LidarPointCloud::GetAssetClass() const
{
	return ULidarPointCloud::StaticClass();
}

namespace MenuExtension_LidarPointCloud
{

	void ExecuteMerge(TArray<ULidarPointCloud*> PointClouds)
	{
		if (PointClouds.Num() < 2)
		{
			return;
		}

		FString PackageName = PointClouds[0]->GetOutermost()->GetName() + TEXT("_Merged");
		UPackage* MergedCloudPackage = UPackageTools::FindOrCreatePackageForAssetType(FName(*PackageName), ULidarPointCloud::StaticClass());
		if (IsValid(MergedCloudPackage))
		{
			MergedCloudPackage->SetPackageFlags(PKG_NewlyCreated);

			ULidarPointCloud* PC = NewObject<ULidarPointCloud>(MergedCloudPackage, FName(*FPaths::GetBaseFilename(MergedCloudPackage->GetName())), EObjectFlags::RF_Public | EObjectFlags::RF_Standalone | EObjectFlags::RF_Transactional);

			FLidarPointCloudEditorHelper::MergeLidar(PC, PointClouds);
		}
	}

	void ExecuteAlign(TArray<ULidarPointCloud*> PointClouds)
	{
		if (PointClouds.Num() < 2)
		{
			return;
		}

		FScopedSlowTask ProgressDialog(1, LOCTEXT("Align", "Aligning Point Clouds..."));
		ProgressDialog.MakeDialog();
		ProgressDialog.EnterProgressFrame(1.f);
		ULidarPointCloud::AlignClouds(PointClouds);
	}

	void ExecuteCollision(TArray<ULidarPointCloud*> PointClouds)
	{
		for (ULidarPointCloud* PC : PointClouds)
		{
			PC->BuildCollision();
		}
	}

	void ExecuteNormals(TArray<ULidarPointCloud*> PointClouds)
	{
		for (ULidarPointCloud* PC : PointClouds)
		{
			// Data needs to be persistently loaded to calculate normals
			if (!PC->IsFullyLoaded())
			{
				PC->LoadAllNodes();
			}

			PC->CalculateNormals(nullptr, nullptr);
		}
	}



	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(ULidarPointCloud::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					TArray<FAssetData> Assets = CBContext->GetSelectedAssetsOfType(ULidarPointCloud::StaticClass());

					if (Assets.Num() > 1)
					{
						InSection.AddMenuEntry("LidarPointCloud_Merge", 
							LOCTEXT("LidarPointCloud_Merge", "Merge Selected"), 
							LOCTEXT("LidarPointCloud_MergeTooltip", "Merges selected point cloud assets."), 
							FSlateIcon("LidarPointCloudStyle", "LidarPointCloudEditor.ToolkitMerge"),
							FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
							{
								if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(Context))
								{
									ExecuteMerge(CBContext->LoadSelectedObjects<ULidarPointCloud>());
								}
							}));

						InSection.AddMenuEntry("LidarPointCloud_Align",
							LOCTEXT("LidarPointCloud_Align", "Align Selected"),
							LOCTEXT("LidarPointCloud_AlignTooltip", "Aligns selected point cloud assets."),
							FSlateIcon("LidarPointCloudStyle", "LidarPointCloudEditor.ToolkitAlign"),
							FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
							{
								if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(Context))
								{
									ExecuteAlign(CBContext->LoadSelectedObjects<ULidarPointCloud>());
								}
							}));
					}

					InSection.AddMenuEntry("LidarPointCloud_BuildCollision",
						LOCTEXT("LidarPointCloud_BuildCollision", "Build Collision"),
						LOCTEXT("LidarPointCloud_BuildCollisionTooltip", "Builds collision for all selected point cloud assets."),
						FSlateIcon("LidarPointCloudStyle", "LidarPointCloudEditor.ToolkitCollision"),
						FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
						{
							if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(Context))
							{
								ExecuteCollision(CBContext->LoadSelectedObjects<ULidarPointCloud>());
							}
						}));

					InSection.AddMenuEntry("LidarPointCloud_CalculateNormals",
						LOCTEXT("LidarPointCloud_CalculateNormals", "Calculate Normals"),
						LOCTEXT("LidarPointCloud_CalculateNormalsTooltip", "Calculates normals for all selected point cloud assets."),
						FSlateIcon("LidarPointCloudStyle", "LidarPointCloudEditor.ToolkitNormals"),
						FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
						{
							if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(Context))
							{
								ExecuteNormals(CBContext->LoadSelectedObjects<ULidarPointCloud>());
							}
						}));
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE