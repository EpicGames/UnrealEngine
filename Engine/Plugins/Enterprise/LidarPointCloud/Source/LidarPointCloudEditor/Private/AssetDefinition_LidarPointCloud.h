// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_LidarPointCloud.generated.h"


UCLASS()
class UAssetDefinition_LidarPointCloud : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override { return FColor(0, 128, 128); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override { return true; }
	// UAssetDefinition End
};
