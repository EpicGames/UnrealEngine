// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEdMode.h"

#include "ContentBrowserModule.h"
#include "EditorModeManager.h"
#include "InteractiveToolManager.h"
#include "LevelEditorViewport.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudActor.h"
#include "LidarPointCloudEditorCommands.h"
#include "LidarPointCloudEditorTools.h"
#include "LidarPointCloudEdModeToolkit.h"
#include "Selection.h"

#define LOCTEXT_NAMESPACE "LidarPointCloudEdMode"

namespace FLidarEditorModes
{
	const FEditorModeID EM_Lidar(TEXT("EM_Lidar"));
}

ULidarEditorMode::ULidarEditorMode()
	: Super()
{
	Info = FEditorModeInfo(
		FLidarEditorModes::EM_Lidar,
		NSLOCTEXT("EditorModes", "LidarMode", "Lidar"),
		FSlateIcon(FLidarPointCloudStyle::GetStyleSetName(), "ClassThumbnail.LidarPointCloud", "ClassIcon.LidarPointCloud"),
		true
		);
}

void ULidarEditorMode::Enter()
{
	Super::Enter();
	
	GEditor->SelectNone(true, true);

	const FLidarPointCloudEditorCommands& Commands = FLidarPointCloudEditorCommands::Get();

#define REGISTER_TOOL(Tool) RegisterTool(Commands.Toolkit##Tool, TEXT("Lidar"#Tool"Tool"), NewObject<ULidarEditorToolBuilder##Tool>())
	REGISTER_TOOL(Select);
	REGISTER_TOOL(BoxSelection);
	REGISTER_TOOL(PolygonalSelection);
	REGISTER_TOOL(LassoSelection);
	REGISTER_TOOL(PaintSelection);
#undef REGISTER_TOOL

	GetToolManager()->SelectActiveToolType(EToolSide::Mouse, "LidarSelectTool");
	GetToolManager()->ActivateTool(EToolSide::Mouse);
}

bool ULidarEditorMode::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	return InActor && (InActor->IsA(ALidarPointCloudActor::StaticClass()) || InActor->IsA(ALidarClippingVolume::StaticClass()));
}

void ULidarEditorMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FLidarPointCloudEdModeToolkit);
}

bool ULidarEditorMode::GetPivotForOrbit(FVector& OutPivot) const
{
	if (GCurrentLevelEditingViewportClient)
	{
		OutPivot = GCurrentLevelEditingViewportClient->GetViewTransform().GetLookAt();
		return true;
	}
	return false;
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> ULidarEditorMode::GetModeCommands() const
{
	return FLidarPointCloudEditorCommands::GetToolkitCommands();
}

void ULidarEditorMode::CancelActiveToolAction()
{
	if (GetToolManager()->HasAnyActiveTool())
	{
		if (IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(GetToolManager()->GetActiveTool(EToolSide::Mouse)))
		{
			CancelAPI->ExecuteNestedCancelCommand();
		}
	}
}

void ULidarEditorMode::BindCommands()
{
	const FLidarPointCloudEditorCommands& Commands = FLidarPointCloudEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(
		Commands.ToolktitCancelSelection,
		FExecuteAction::CreateUObject(this, &ULidarEditorMode::CancelActiveToolAction),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);
}

void ULidarEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	if(const ULidarEditorToolBase* LidarTool = Cast<ULidarEditorToolBase>(Tool))
	{
		GetToolManager()->DisplayMessage(LidarTool->GetToolMessage(), EToolMessageLevel::UserNotification);

		if(FLidarPointCloudEdModeToolkit* LidarToolkit = (FLidarPointCloudEdModeToolkit*)Toolkit.Get())
		{
			LidarToolkit->SetActorSelection(LidarTool->IsActorSelection());
			LidarToolkit->SetBrushTool(Cast<ULidarEditorToolPaintSelection>(Tool));
		}
	}
}

void ULidarEditorMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	if(Tool && Tool->IsA(ULidarEditorToolSelectionBase::StaticClass()))
	{
		FLidarPointCloudEditorHelper::ClearSelection();
	}
}

#undef LOCTEXT_NAMESPACE
