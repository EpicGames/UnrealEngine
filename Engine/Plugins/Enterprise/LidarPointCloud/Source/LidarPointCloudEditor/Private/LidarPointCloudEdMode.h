// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Tools/UEdMode.h"
#include "LidarPointCloudEdMode.generated.h"

namespace FLidarEditorModes
{
	LIDARPOINTCLOUDEDITOR_API extern const FEditorModeID EM_Lidar;
}

/**
* Lidar editor mode
*/
UCLASS()
class ULidarEditorMode : public UBaseLegacyWidgetEdMode
{
public:
	GENERATED_BODY()
	
	ULidarEditorMode();

	virtual void Enter() override;
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;
	virtual void CreateToolkit() override;
	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	virtual TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetModeCommands() const override;
	virtual void BindCommands() override;
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
private:
	void CancelActiveToolAction();
};