// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEdModeToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "LidarPointCloudEditorCommands.h"
#include "LidarPointCloudEditorTools.h"
#include "LidarPointCloudEdMode.h"
#include "Selection.h"
#include "SlateOptMacros.h"
#include "StatusBarSubsystem.h"
#include "Toolkits/AssetEditorModeUILayer.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "LidarEditMode"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SLidarEditorWidget::Construct(const FArguments& InArgs)
{
	LidarEditorMode = (ULidarEditorMode*)GLevelEditorModeTools().GetActiveMode(FLidarEditorModes::EM_Lidar);
	
	// Everything (or almost) uses this padding, change it to expand the padding.
	FMargin StandardPadding(5.f, 2.f);
	FMargin HeaderPadding(8.f, 4.f);

	FSlateFontInfo HeaderFont = FAppStyle::GetFontStyle(TEXT("DetailsView.CategoryFontStyle")); HeaderFont.Size = 11;
	FSlateFontInfo StandardFont = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));
	FSlateFontInfo LabelFont = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")); LabelFont.Size = 9;

#define HEADEREX(ContText, Category) +SVerticalBox::Slot().AutoHeight().Padding(FMargin(0.0f, 5.0f))[SNew(SHeaderRow)+SHeaderRow::Column(ContText).HAlignCell(HAlign_Left).FillWidth(1).HeaderContentPadding(HeaderPadding)[SNew(STextBlock).Text(LOCTEXT(Category, ContText)).Font(HeaderFont)]]
#define HEADER(ContText) HEADEREX(ContText, ContText"Header")
#define BUTTON(Category, Context, Tooltip, Action) +SHorizontalBox::Slot().Padding(StandardPadding).FillWidth(0.5f)[SNew(SButton).HAlign(HAlign_Center).Text(LOCTEXT(Category,Context)).OnClicked_Lambda([this]{ Action return FReply::Handled(); }).ToolTipText(LOCTEXT(Category"Tip",Tooltip))]
#define BUTTON_ANY(Category, Context, Tooltip, Action) +SHorizontalBox::Slot().Padding(StandardPadding).FillWidth(0.5f)[SNew(SButton).HAlign(HAlign_Center).Text(LOCTEXT(Category,Context)).OnClicked_Lambda([this]{ Action return FReply::Handled(); }).IsEnabled(this, &SLidarEditorWidget::IsAnySelection).ToolTipText(LOCTEXT(Category"Tip",Tooltip))]
#define BUTTON_POINTS(Category, Context, Tooltip, Action) +SHorizontalBox::Slot().Padding(StandardPadding).FillWidth(0.5f)[SNew(SButton).HAlign(HAlign_Center).Text(LOCTEXT(Category,Context)).OnClicked_Lambda([this]{ Action return FReply::Handled(); }).IsEnabled(this, &SLidarEditorWidget::IsPointSelection).ToolTipText(LOCTEXT(Category"Tip",Tooltip))]
#define BUTTON_ACTORS(Category, Context, Tooltip, Action) +SHorizontalBox::Slot().Padding(StandardPadding).FillWidth(0.5f)[SNew(SButton).HAlign(HAlign_Center).Text(LOCTEXT(Category,Context)).OnClicked_Lambda([this]{ Action return FReply::Handled(); }).IsEnabled(this, &SLidarEditorWidget::IsActorSelection).ToolTipText(LOCTEXT(Category"Tip",Tooltip))]
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ANY("SelectionClear", "Clear", "Deselect all actors and points",
			{
				FLidarPointCloudEditorHelper::ClearActorSelection();
                FLidarPointCloudEditorHelper::ClearSelection();
			})
			BUTTON("SelectionInvert", "Invert", "Invert selected actors or points",
			{
				if(IsActorSelection())
				{
					FLidarPointCloudEditorHelper::InvertActorSelection();
				}
				else
				{
					FLidarPointCloudEditorHelper::InvertSelection();
				}
			})
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 5.0f))
		[
			SNew(SHeaderRow)
			.Visibility(this, &SLidarEditorWidget::GetBrushVisibility)
			+SHeaderRow::Column("Brush")
			.HAlignCell(HAlign_Left)
			.FillWidth(1)
			.HeaderContentPadding(HeaderPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BrushHeader", "Brush"))
				.Font(HeaderFont)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SLidarEditorWidget::GetBrushVisibility)
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BrushRadius", "Brush Radius"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(StandardFont)
				.AllowSpin(true)
				.MinValue(0.0f)
				.MaxValue(65536.0f)
				.MaxSliderValue(4096.0f)
				.MinDesiredValueWidth(50.0f)
				.SliderExponent(3.0f)
				.Value_Lambda([this]
				{
					return BrushTool ? BrushTool->BrushRadius : 0;
				})
				.OnValueChanged_Lambda([this](float Value)
				{
					if(BrushTool)
					{
						BrushTool->BrushRadius = Value;
					}
				})
			]
		]
		HEADER("Cleanup")
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_POINTS("CleanupHideSelected", "Hide Selected", "Hide all selected points", {FLidarPointCloudEditorHelper::HideSelected();})
			BUTTON("CleanupResetVisibility", "Reset Visibility", "Mark all hidden points as visible", {FLidarPointCloudEditorHelper::ResetVisibility();})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_POINTS("CleanupDeleteSelected", "Delete Selected", "Permanently delete all selected points", {FLidarPointCloudEditorHelper::DeleteSelected();})
			BUTTON("CleanupDeleteHidden", "Delete Hidden", "Permanently delete all hidden points", {FLidarPointCloudEditorHelper::DeleteHidden();})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_POINTS("CleanupCropToSelection", "Crop To Selection", "Hides all points not within selection",
			{
				FLidarPointCloudEditorHelper::InvertSelection();
				FLidarPointCloudEditorHelper::HideSelected();
			})
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SSpacer)
			]
		]
		HEADER("Collisions")
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("CollisionsErrorTooltip", "Determines the maximum error (in cm) of the collision. Lower values will require more time to build."))
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CollisionsError", "Max Error"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(StandardFont)
				.AllowSpin(true)
				.MinValue(0.0f)
				.MaxValue(65536.0f)
				.MaxSliderValue(512.0f)
				.MinDesiredValueWidth(50.0f)
				.Value_Lambda([this]{ return MaxCollisionError; })
				.OnValueChanged_Lambda([this](float Value) { MaxCollisionError = Value; })
				.IsEnabled(this, &SLidarEditorWidget::IsActorSelection)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ACTORS("CollisionsAdd", "Add Collision", "Generate collision for selected assets",
			{
				FLidarPointCloudEditorHelper::SetCollisionErrorForSelection(MaxCollisionError);
				FLidarPointCloudEditorHelper::BuildCollisionForSelection();
			})
			BUTTON_ACTORS("CollisionsRemove", "Remove Collision", "Removes any existing collisions from selected assets", {FLidarPointCloudEditorHelper::RemoveCollisionForSelection();})
		]
		HEADER("Normals")
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("NormalsQualityTooltip", "Higher values will generally result in more accurate calculations, at the expense of time"))
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NormalsQuality", "Quality"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SNumericEntryBox<int32>)
				.Font(StandardFont)
				.AllowSpin(true)
				.MinValue(0)
				.MaxValue(100)
				.Value_Lambda([this]{ return NormalsQuality; })
				.OnValueChanged_Lambda([this](int32 Value) { NormalsQuality = Value; })
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("NormalsNoiseToleranceTooltip", "Higher values are less susceptible to noise, but will most likely lose finer details, especially around hard edges."))
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NormalsNoiseTolerance", "Noise Tolerance"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(StandardFont)
				.AllowSpin(true)
				.MinValue(0.0f)
				.MaxValue(20.0f)
				.MaxSliderValue(5.0f)
				.MinDesiredValueWidth(50.0f)
				.Value_Lambda([this]{ return NormalsNoiseTolerance; })
				.OnValueChanged_Lambda([this](float Value) { NormalsNoiseTolerance = Value; })
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ANY("NormalsCalculate", "Calculate Normals", "Calculates normal vectors for all selected assets or points",
			{
				FLidarPointCloudEditorHelper::SetNormalsQuality(NormalsQuality, NormalsNoiseTolerance);
				if(IsPointSelection())
				{
					FLidarPointCloudEditorHelper::CalculateNormalsForSelection();
				}
				else
				{
					FLidarPointCloudEditorHelper::CalculateNormals();
				}
			})
		]
		HEADER("Meshing")
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("MeshingErrorTooltip", "Determines the maximum error (in cm) of the resulting mesh. Lower values will require more time to build."))
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MeshingError", "Max Error"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(StandardFont)
				.AllowSpin(true)
				.MinValue(0.0f)
				.MaxValue(65536.0f)
				.MaxSliderValue(512.0f)
				.MinDesiredValueWidth(50.0f)
				.Value_Lambda([this]{ return MaxMeshingError; })
				.OnValueChanged_Lambda([this](float Value) { MaxMeshingError = Value; })
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("MeshingMergeTooltip", "When enabled, all elements will be combined into one big mesh"))
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MeshingMerge", "Merge Meshes"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]{ return bMergeMeshes ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState Value) { bMergeMeshes = Value == ECheckBoxState::Checked; })
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("MeshingRetainTooltip", "When enabled, all generated meshes will retain the transforms of their source lidar data"))
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MeshingRetain", "Retain Transform"))
				.Font(LabelFont)
			]
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]{ return !bMergeMeshes && bRetainTransform ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState Value) { bRetainTransform = Value == ECheckBoxState::Checked; })
				.IsEnabled_Lambda([this] { return !bMergeMeshes; })
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ANY("MeshingBuild", "Create Static Mesh", "Generates a static mesh from selected assets or points", {
				FLidarPointCloudEditorHelper::MeshSelected(IsPointSelection(), MaxMeshingError, bMergeMeshes, !bMergeMeshes && bRetainTransform);
			})
		]
		HEADER("Alignment")
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ACTORS("AlignmentCoords", "Original Coordinates", "Aligns all selected assets to their original coordinates", {FLidarPointCloudEditorHelper::SetOriginalCoordinateForSelection();})
			BUTTON_ACTORS("AlignmentOrigin", "World Origin", "Aligns all selected assets relative to each other, while keeping the overall pivot around 0,0,0", {FLidarPointCloudEditorHelper::AlignSelectionAroundWorldOrigin();})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ACTORS("AlignmentReset", "Reset Alignment", "Resets the pivot of all selected assets to 0,0,0", {FLidarPointCloudEditorHelper::CenterSelection();})
			+ SHorizontalBox::Slot()
			.Padding(StandardPadding)
			.FillWidth(0.5f)
			[
				SNew(SSpacer)
			]
		]
		HEADEREX("Merge & Extract", "MergeExtract")
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_POINTS("MergeExtractSelection", "Extract Selection", "Extracts the selected points as separate assets, removing them from the original one", {FLidarPointCloudEditorHelper::Extract();})
			BUTTON_POINTS("MergeExtractCopy", "Extract as Copy", "Extracts the selected points as separate assets, retaining them in the original one", {FLidarPointCloudEditorHelper::ExtractAsCopy();})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			BUTTON_ACTORS("MergeExtractActors", "Merge Actors", "Replaces all individual selected assets with a single new actor", {FLidarPointCloudEditorHelper::MergeSelectionByComponent(true);})
			BUTTON_ACTORS("MergeExtractAssets", "Merge Assets", "Combines all selected assets into a single, large new asset", {FLidarPointCloudEditorHelper::MergeSelectionByData(true);})
		]
	];

#undef BUTTON_ACTORS
#undef BUTTON_POINTS
#undef BUTTON
#undef HEADER
#undef HEADEREX
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool SLidarEditorWidget::IsActorSelection() const
{
	return bActorSelection && FLidarPointCloudEditorHelper::AreLidarActorsSelected() && !FLidarPointCloudEditorHelper::AreLidarPointsSelected();
}

bool SLidarEditorWidget::IsPointSelection() const
{
	return !bActorSelection && FLidarPointCloudEditorHelper::AreLidarPointsSelected();
}

bool SLidarEditorWidget::IsAnySelection() const
{
	return FLidarPointCloudEditorHelper::AreLidarActorsSelected() || FLidarPointCloudEditorHelper::AreLidarPointsSelected();
}

FLidarPointCloudEdModeToolkit::~FLidarPointCloudEdModeToolkit()
{
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.RemoveAll(this);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.RemoveAll(this);
}

void FLidarPointCloudEdModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	EditorWidget = SNew(SLidarEditorWidget);
	
	FModeToolkit::Init(InitToolkitHost, InOwningMode);
	
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.AddSP(this, &FLidarPointCloudEdModeToolkit::SetActiveToolMessage);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.AddSP(this, &FLidarPointCloudEdModeToolkit::SetActiveToolMessage);
}

FName FLidarPointCloudEdModeToolkit::GetToolkitFName() const
{
	return FName("LidarEditMode");
}

FText FLidarPointCloudEdModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT( "ToolkitName", "Lidar" );
}

void FLidarPointCloudEdModeToolkit::GetToolPaletteNames(TArray<FName>& InPaletteName) const
{
	InPaletteName.Add(LidarEditorPalletes::Manage);
}

FText FLidarPointCloudEdModeToolkit::GetToolPaletteDisplayName(FName PaletteName) const
{
	return LOCTEXT("LidarMode_Manage", "Manage");
}

FText FLidarPointCloudEdModeToolkit::GetActiveToolDisplayName() const
{
	if (UInteractiveTool* ActiveTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left))
	{
		return ActiveTool->GetClass()->GetDisplayNameText();
	}

	return LOCTEXT("LidarNoActiveTool", "LidarNoActiveTool");
}

FText FLidarPointCloudEdModeToolkit::GetActiveToolMessage() const
{
	return ActiveToolMessageCache;
}

void FLidarPointCloudEdModeToolkit::SetActiveToolMessage(const FText& Message)
{
	ActiveToolMessageCache = Message;
	
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), Message);
	}

	ActiveToolMessageHandle.Reset();
}

void FLidarPointCloudEdModeToolkit::SetActorSelection(bool bNewActorSelection)
{
	if(EditorWidget.IsValid())
	{
		EditorWidget->bActorSelection = bNewActorSelection;
	}
}

void FLidarPointCloudEdModeToolkit::SetBrushTool(ULidarEditorToolPaintSelection* NewBrushTool)
{
	if(EditorWidget.IsValid())
	{
		EditorWidget->BrushTool = NewBrushTool;
	}
}

#undef LOCTEXT_NAMESPACE
