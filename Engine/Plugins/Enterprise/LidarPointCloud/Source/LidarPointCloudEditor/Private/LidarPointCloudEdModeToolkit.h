// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Toolkits/BaseToolkit.h"
#include "EditorModeManager.h"
#include "StatusBarSubsystem.h"

class ULidarEditorMode;
class ULidarEditorToolPaintSelection;

namespace LidarEditorPalletes
{
	static const FName Manage(TEXT("ToolMode_Manage")); 
}

class SLidarEditorWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLidarEditorWidget) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	bool IsActorSelection() const;
	bool IsPointSelection() const;
	bool IsAnySelection() const;
	bool IsBrushVisible() const { return BrushTool != nullptr; }

	EVisibility GetActorVisibility() const { return IsActorSelection() ? EVisibility::Visible : EVisibility::Collapsed; }
	EVisibility GetPointVisibility() const { return IsPointSelection() ? EVisibility::Visible : EVisibility::Collapsed; }
	EVisibility GetAnyVisibility() const { return IsAnySelection() ? EVisibility::Visible : EVisibility::Collapsed; }
	EVisibility GetBrushVisibility() const { return IsBrushVisible() ? EVisibility::Visible : EVisibility::Collapsed; }

private:	
	float MaxCollisionError = 0.0f;
	float MaxMeshingError = 0.0f;
	bool bMergeMeshes = true;
	bool bRetainTransform = true;
	int32 NormalsQuality = 40;
	float NormalsNoiseTolerance = 1.0f;

	ULidarEditorMode* LidarEditorMode = nullptr;
	ULidarEditorToolPaintSelection* BrushTool = nullptr;
	bool bActorSelection = false;

	friend class FLidarPointCloudEdModeToolkit;
};

/**
 * Public interface to Lidar Edit mode.
 */
class FLidarPointCloudEdModeToolkit : public FModeToolkit
{
public:
	FLidarPointCloudEdModeToolkit(){}
	~FLidarPointCloudEdModeToolkit();
	
	/** Initializes the Lidar mode toolkit */
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const override;
	virtual bool HasIntegratedToolPalettes() const override { return true; }	
	virtual FText GetActiveToolDisplayName() const override;
	virtual FText GetActiveToolMessage() const override;
	virtual TSharedPtr<SWidget> GetInlineContent() const override { return EditorWidget; }
	
	void SetActiveToolMessage(const FText& Message);
	void SetActorSelection(bool bNewActorSelection);
	void SetBrushTool(ULidarEditorToolPaintSelection* NewBrushTool);

private:
	FStatusBarMessageHandle ActiveToolMessageHandle;
	FText ActiveToolMessageCache;
	TSharedPtr<SLidarEditorWidget> EditorWidget;
};