// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEditor.h"
#include "LidarPointCloudEditorViewport.h"
#include "LidarPointCloudEditorViewportClient.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudShared.h"
#include "ILidarPointCloudEditorModule.h"
#include "LidarPointCloudEditorCommands.h"
#include "LidarPointCloudEditorHelper.h"

#include "Misc/ScopedSlowTask.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SSingleObjectDetailsPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Docking/SDockTab.h"
#include "UObject/Package.h"
#include "Framework/Commands/UICommandList.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "LidarPointCloudEditor"

const FName PointCloudEditorAppName = FName(TEXT("LidarPointCloudEditorApp"));

const FName ViewportTabId(TEXT("Viewport"));

//////////////////////////////////////////////////////////////////////////
// FLidarPointCloudEditor

FLidarPointCloudEditor::FLidarPointCloudEditor()
	: PointCloudBeingEdited(nullptr)
	, bEditMode(false)
{
}

FLidarPointCloudEditor::~FLidarPointCloudEditor()
{
	// Unregister from the cloud before closing
	if (PointCloudBeingEdited)
	{
		PointCloudBeingEdited->OnPointCloudRebuilt().RemoveAll(this);
		PointCloudBeingEdited->OnPreSaveCleanup().RemoveAll(this);
	}
}

void FLidarPointCloudEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_LidarPointCloudEditor", "LiDAR Point Cloud Editor"));
	
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	
	InTabManager->RegisterTabSpawner(ViewportTabId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	{
		TSharedRef<SDockTab> SpawnedTab =
			SNew(SDockTab)
			.Label(LOCTEXT("ViewportTab_Title", "Viewport"))
			[
				Viewport.ToSharedRef()
			];

		Viewport->SetParentTab(SpawnedTab);

		return SpawnedTab;
	}))
	.SetDisplayName(LOCTEXT("ViewportTabLabel", "Viewport"))
	.SetGroup(WorkspaceMenuCategory.ToSharedRef())
	.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
}

void FLidarPointCloudEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ViewportTabId);
}

FName FLidarPointCloudEditor::GetToolkitFName() const { return FName("LidarPointCloudEditor"); }
FText FLidarPointCloudEditor::GetBaseToolkitName() const { return LOCTEXT("LidarPointCloudEditorAppLabel", "LiDAR Point Cloud Editor"); }
FText FLidarPointCloudEditor::GetToolkitToolTipText() const { return FAssetEditorToolkit::GetToolTipTextForObject(PointCloudBeingEdited); }
FLinearColor FLidarPointCloudEditor::GetWorldCentricTabColorScale() const { return FLinearColor::White; }
FString FLidarPointCloudEditor::GetWorldCentricTabPrefix() const { return TEXT("LidarPointCloudEditor"); }

FText FLidarPointCloudEditor::GetToolkitName() const
{
	const bool bDirtyState = PointCloudBeingEdited->GetOutermost()->IsDirty();

	FFormatNamedArguments Args;
	Args.Add(TEXT("PointCloudName"), FText::FromString(PointCloudBeingEdited->GetName()));
	Args.Add(TEXT("DirtyState"), bDirtyState ? FText::FromString(TEXT("*")) : FText::GetEmpty());
	return FText::Format(LOCTEXT("LidarPointCloudEditorToolkitName", "{PointCloudName}{DirtyState}"), Args);
}

void FLidarPointCloudEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (IsValid(PointCloudBeingEdited))
	{
		Collector.AddReferencedObject(PointCloudBeingEdited);
	}
}

void FLidarPointCloudEditor::InitPointCloudEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class ULidarPointCloud* InitPointCloud)
{
	PointCloudBeingEdited = InitPointCloud;

	TSharedPtr<FLidarPointCloudEditor> PointCloudEditor = SharedThis(this);

	Viewport = SNew(SLidarPointCloudEditorViewport).PointCloudEditor(SharedThis(this)).ObjectToEdit(PointCloudBeingEdited);

	// Default layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_LidarPointCloudEditor_Layout_v4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(ViewportTabId, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			)
		);

	// Initialize the asset editor
	InitAssetEditor(Mode, InitToolkitHost, PointCloudEditorAppName, StandaloneDefaultLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, InitPointCloud);

	ExtendToolBar();

	RegenerateMenusAndToolbars();
}

void FLidarPointCloudEditor::ExtendToolBar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FLidarPointCloudEditor* ThisEditor)
		{
			const FLidarPointCloudEditorCommands* Commands = &FLidarPointCloudEditorCommands::Get();

			ToolbarBuilder.BeginSection("Camera");
			{
				ToolbarBuilder.AddToolBarButton(Commands->ResetCamera, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "StaticMeshEditor.ResetCamera"));
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, Viewport->GetCommandList(), FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this));
	AddToolbarExtender(ToolbarExtender);

	AddToolbarExtender(ILidarPointCloudEditorModule::Get().GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

#undef LOCTEXT_NAMESPACE