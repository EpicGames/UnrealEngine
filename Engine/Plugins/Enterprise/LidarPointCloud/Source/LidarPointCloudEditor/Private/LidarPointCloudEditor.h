// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Text/STextBlock.h"
#include "AssetRegistry/AssetData.h"
#include "ConvexVolume.h"
#include "LidarPointCloudShared.h"

class ULidarPointCloud;
struct FLidarPointCloudPoint;
class SWidget;
class SLidarPointCloudEditorViewport;

class FLidarPointCloudEditor : public FAssetEditorToolkit, public FGCObject
{
private:
	TObjectPtr<ULidarPointCloud> PointCloudBeingEdited;

	TArray64<FLidarPointCloudPoint*> SelectedPoints;
	
	bool bEditMode;

	/** Preview Viewport widget */
	TSharedPtr<SLidarPointCloudEditorViewport> Viewport;

public:
	FLidarPointCloudEditor();
	~FLidarPointCloudEditor();

	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	virtual void OnToolkitHostingStarted(const TSharedRef<class IToolkit>& Toolkit) override {}
	virtual void OnToolkitHostingFinished(const TSharedRef<class IToolkit>& Toolkit) override {}
	// End of FAssetEditorToolkit

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FLidarPointCloudEditor");
	}
	// End of FSerializableObject interface

	void InitPointCloudEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class ULidarPointCloud* InitPointCloud);

	ULidarPointCloud* GetPointCloudBeingEdited() const { return PointCloudBeingEdited; }

	TSharedPtr<SLidarPointCloudEditorViewport> GetViewport() { return Viewport; }

private:
	/** Builds the Point Cloud Editor toolbar. */
	void ExtendToolBar();
};