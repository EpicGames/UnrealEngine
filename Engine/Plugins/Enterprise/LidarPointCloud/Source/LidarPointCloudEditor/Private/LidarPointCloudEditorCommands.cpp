// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEditorCommands.h"

#include "LidarPointCloudEdModeToolkit.h"

#define LOCTEXT_NAMESPACE "LidarPointCloudEditor"

//////////////////////////////////////////////////////////////////////////
// FLidarPointCloudEditorCommands

void FLidarPointCloudEditorCommands::RegisterCommands()
{
	UI_COMMAND(SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowBounds, "Bounds", "Toggles display of the bounds of the point cloud.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowCollision, "Collision", "Toggles display of the collision of the point cloud.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowNodes, "Nodes", "Toggles display of the nodes of the point cloud.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ResetCamera, "Reset Camera", "Resets the camera to focus on the point cloud.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Center, "Center", "Enable, to center the point cloud asset\nDisable, to use original coordinates.", EUserInterfaceActionType::ToggleButton, FInputChord());

	// TOOLKIT COMMANDS
	TArray<TSharedPtr<FUICommandInfo, ESPMode::ThreadSafe>>& EditCommands = ToolkitCommands.FindOrAdd(LidarEditorPalletes::Manage);
	UI_COMMAND(ToolkitSelect, "Object", "Select point cloud assets", EUserInterfaceActionType::ToggleButton, FInputChord());
	EditCommands.Add(ToolkitSelect);
	UI_COMMAND(ToolkitBoxSelection, "Marquee", "Uses box to select points.", EUserInterfaceActionType::ToggleButton, FInputChord());
	EditCommands.Add(ToolkitBoxSelection);
	UI_COMMAND(ToolkitPolygonalSelection, "Polygon", "Uses custom polygon to select points.", EUserInterfaceActionType::ToggleButton, FInputChord());
	EditCommands.Add(ToolkitPolygonalSelection);
	UI_COMMAND(ToolkitLassoSelection, "Lasso", "Uses custom drawn shape to select points.", EUserInterfaceActionType::ToggleButton, FInputChord());
	EditCommands.Add(ToolkitLassoSelection);
	UI_COMMAND(ToolkitPaintSelection, "Paint", "Uses adjustable paint brush to select points.", EUserInterfaceActionType::ToggleButton, FInputChord());
	EditCommands.Add(ToolkitPaintSelection);

	UI_COMMAND(ToolktitCancelSelection, "Cancel Selection", "Cancel the active tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
