// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "LidarPointCloudStyle.h"

class FLidarPointCloudEditorCommands : public TCommands<FLidarPointCloudEditorCommands>
{
public:
	FLidarPointCloudEditorCommands()
		: TCommands<FLidarPointCloudEditorCommands>(
			TEXT("LidarPointCloudEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "LidarPointCloudEditor", "LiDAR Point Cloud Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FLidarPointCloudStyle::GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

	static TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetToolkitCommands()
	{
		return Get().ToolkitCommands;
	}

public:
	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowBounds;
	TSharedPtr<FUICommandInfo> SetShowCollision;
	TSharedPtr<FUICommandInfo> SetShowNodes;
	TSharedPtr<FUICommandInfo> ResetCamera;
	TSharedPtr<FUICommandInfo> Center;

	// TOOLKIT COMMANDS

	TSharedPtr<FUICommandInfo> ToolkitSelect;
	TSharedPtr<FUICommandInfo> ToolkitBoxSelection;
	TSharedPtr<FUICommandInfo> ToolkitPolygonalSelection;
	TSharedPtr<FUICommandInfo> ToolkitLassoSelection;
	TSharedPtr<FUICommandInfo> ToolkitPaintSelection;
	TSharedPtr<FUICommandInfo> ToolktitCancelSelection;

private:
	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> ToolkitCommands;
};