// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEditorHelper.h"
#include "ContentBrowserModule.h"
#include "GeomTools.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditorViewport.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudActor.h"
#include "LidarPointCloudComponent.h"
#include "MeshDescription.h"
#include "SceneView.h"
#include "Selection.h"
#include "StaticMeshAttributes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMeshActor.h"
#include "Misc/ScopedSlowTask.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshResources.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "LidarPointCloudEditorHelper"

class FContentBrowserModule;

namespace
{
	FString GetSaveAsLocation()
	{
		// Initialize SaveAssetDialog config
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SelectDestination", "Select Destination");
		SaveAssetDialogConfig.DefaultPath = "/Game";
		SaveAssetDialogConfig.AssetClassNames.Add(ULidarPointCloud::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		return ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	}

	ULidarPointCloud* CreateNewAsset_Internal()
	{
		ULidarPointCloud* NewPointCloud = nullptr;

		const FString SaveObjectPath = GetSaveAsLocation();
		if (!SaveObjectPath.IsEmpty())
		{
			// Attempt to load existing asset first
			NewPointCloud = FindObject<ULidarPointCloud>(nullptr, *SaveObjectPath);

			// Proceed to creating a new asset, if needed
			if (!NewPointCloud)
			{
				const FString PackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
				const FString ObjectName = FPackageName::ObjectPathToObjectName(SaveObjectPath);

				NewPointCloud = NewObject<ULidarPointCloud>(CreatePackage(*PackageName), ULidarPointCloud::StaticClass(), FName(*ObjectName), EObjectFlags::RF_Public | EObjectFlags::RF_Standalone);

				FAssetRegistryModule::AssetCreated(NewPointCloud);
				NewPointCloud->MarkPackageDirty();
			}		
		}

		return NewPointCloud;
	}

	void ProcessSelection(TFunction<void(ALidarPointCloudActor*)> Function)
	{
		if(!Function)
		{
			return;
		}
		
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if(ALidarPointCloudActor* LidarActor = Cast<ALidarPointCloudActor>(*It))
			{
				Function(LidarActor);
			}
		}
	}

	void ProcessAll(TFunction<void(ALidarPointCloudActor*)> Function)
	{
		if(!Function)
		{
			return;
		}
		
		for (TObjectIterator<ALidarPointCloudActor> It; It; ++It)
		{
			ALidarPointCloudActor* LidarActor = Cast<ALidarPointCloudActor>(*It);
			if(IsValid(LidarActor))
			{
				Function(LidarActor);
			}
		}
	}

	TArray<ALidarPointCloudActor*> GetSelectedActors()
	{
		TArray<ALidarPointCloudActor*> Actors;
		
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if(ALidarPointCloudActor* LidarActor = Cast<ALidarPointCloudActor>(*It))
			{
				Actors.Add(LidarActor);
			}
		}

		return Actors;
	}
	
	int32 GetNumLidarActors()
	{
		int32 NumActors = 0;
		
		for (TObjectIterator<ALidarPointCloudActor> It; It; ++It)
		{
			const ALidarPointCloudActor* LidarActor = Cast<ALidarPointCloudActor>(*It);
			if(IsValid(LidarActor))
			{
				++NumActors;
			}
		}

		return NumActors;
	}

	UWorld* GetFirstWorld()
	{
		for (TObjectIterator<ALidarPointCloudActor> It; It; ++It)
		{
			ALidarPointCloudActor* LidarActor = Cast<ALidarPointCloudActor>(*It);
			if(IsValid(LidarActor))
			{
				return LidarActor->GetWorld();
			}
		}
		return nullptr;
	}

	TArray<ULidarPointCloud*> GetSelectedClouds()
	{
		TArray<ULidarPointCloud*> PointClouds;
		
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if(ALidarPointCloudActor* LidarActor = Cast<ALidarPointCloudActor>(*It))
			{
				if(ULidarPointCloud* PointCloud = LidarActor->GetPointCloud())
				{
					PointClouds.Add(PointCloud);
				}
			}
		}

		return PointClouds;
	}

	template<typename T>
	T* SpawnActor(const FString& Name)
	{
		T* NewActor = nullptr;
		if(UWorld* World = GetFirstWorld())
		{
			FActorSpawnParameters ActorSpawnParameters;
			ActorSpawnParameters.Name = FName(*Name);
			NewActor = Cast<T>(World->SpawnActor<T>(ActorSpawnParameters));
			FActorLabelUtilities::SetActorLabelUnique(NewActor, NewActor->GetName());
		}
		return NewActor;
	}
	
	UStaticMesh* CreateStaticMesh()
	{
		UStaticMesh* StaticMesh = nullptr;
		
		// Initialize SaveAssetDialog config
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SelectDestination", "Select Destination");
		SaveAssetDialogConfig.DefaultPath = "/Game";
		SaveAssetDialogConfig.AssetClassNames.Add(UStaticMesh::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		
		const FString SaveObjectPath = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (!SaveObjectPath.IsEmpty())
		{
			// Attempt to load existing asset first
			StaticMesh = FindObject<UStaticMesh>(nullptr, *SaveObjectPath);

			// Proceed to creating a new asset, if needed
			if (!StaticMesh)
			{
				const FString PackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
				const FString ObjectName = FPackageName::ObjectPathToObjectName(SaveObjectPath);

				StaticMesh = NewObject<UStaticMesh>(CreatePackage(*PackageName), UStaticMesh::StaticClass(), FName(*ObjectName), EObjectFlags::RF_Public | EObjectFlags::RF_Standalone);

				FAssetRegistryModule::AssetCreated(StaticMesh);
			}
		}

		return StaticMesh;
	}

	void MeshBuffersToMeshDescription(LidarPointCloudMeshing::FMeshBuffers& MeshBuffers, FMeshDescription& OutMeshDescription)
	{
		FScopeBenchmarkTimer Timer("MeshBuffersToMeshDescription");
		
		FStaticMeshAttributes Attributes(OutMeshDescription);
		Attributes.Register();
		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TVertexInstanceAttributesRef<FVector3f> InstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector4f> InstanceColors = Attributes.GetVertexInstanceColors();

		OutMeshDescription.SuspendVertexInstanceIndexing();
		OutMeshDescription.SuspendEdgeIndexing();
		OutMeshDescription.SuspendPolygonIndexing();
		OutMeshDescription.SuspendPolygonGroupIndexing();
		OutMeshDescription.SuspendUVIndexing();
		OutMeshDescription.ReserveNewVertices(MeshBuffers.Vertices.Num());
		OutMeshDescription.ReserveNewVertexInstances(MeshBuffers.Indices.Num());

		const FPolygonGroupID AllGroupID = OutMeshDescription.CreatePolygonGroup();
		Attributes.GetPolygonGroupMaterialSlotNames().Set(AllGroupID, NAME_None);

		for(LidarPointCloudMeshing::FVertexData& Vertex : MeshBuffers.Vertices)
		{
			VertexPositions.Set(OutMeshDescription.CreateVertex(), Vertex.Position);
		}

		TArray<FVertexInstanceID> CornerInstanceIDs;
		CornerInstanceIDs.SetNumZeroed(3);
		for (uint32* Index = MeshBuffers.Indices.GetData(), *DataEnd = Index + MeshBuffers.Indices.Num(); Index != DataEnd;)
		{
			for(int32 i = 0; i < 3; ++i, ++Index)
			{
				FVertexInstanceID& VertexInstanceID = CornerInstanceIDs[i];
				VertexInstanceID = OutMeshDescription.CreateVertexInstance(*Index);
				LidarPointCloudMeshing::FVertexData& VertexData = MeshBuffers.Vertices[*Index];
				InstanceNormals.Set(VertexInstanceID, VertexData.Normal);
				InstanceColors.Set(VertexInstanceID, FLinearColor(VertexData.Color));
			}
				
			OutMeshDescription.CreateTriangle(AllGroupID, CornerInstanceIDs);
		}
		
		OutMeshDescription.ResumeVertexInstanceIndexing();
		OutMeshDescription.ResumeEdgeIndexing();
		OutMeshDescription.ResumePolygonIndexing();
		OutMeshDescription.ResumePolygonGroupIndexing();
		OutMeshDescription.ResumeUVIndexing();
	}

	void AssignMeshDescriptionToMesh(FMeshDescription& MeshDescription, UStaticMesh* StaticMesh)
	{
		FScopeBenchmarkTimer Timer("AssignMeshDescriptionToMesh");
		
		if(StaticMesh)
		{
			StaticMesh->SetNumSourceModels(1);
			FStaticMeshSourceModel& SourceModelLOD0 = StaticMesh->GetSourceModel(0);
			SourceModelLOD0.BuildSettings.bRecomputeNormals = false;
			SourceModelLOD0.BuildSettings.bRecomputeTangents = false;
			SourceModelLOD0.BuildSettings.bBuildReversedIndexBuffer = false;
			SourceModelLOD0.BuildSettings.bGenerateLightmapUVs = false;

			TArray<FStaticMaterial> StaticMaterials;
			static UMaterialInterface* Material = Cast<UMaterialInterface>(FSoftObjectPath(TEXT("/LidarPointCloud/Materials/M_MeshedCloud.M_MeshedCloud")).TryLoad());
			StaticMaterials.Emplace(Material);
			StaticMesh->SetStaticMaterials(StaticMaterials);
			StaticMesh->BuildFromMeshDescriptions({ &MeshDescription });
			
			StaticMesh->SetLightingGuid(FGuid::NewGuid());
			StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
			StaticMesh->PostEditChange();
		}
	}

	void AssignMeshBuffersToMesh(LidarPointCloudMeshing::FMeshBuffers* MeshBuffers, UStaticMesh* StaticMesh)
	{
		FScopeBenchmarkTimer Timer("AssignMeshBuffersToMesh");
		
		if(StaticMesh && MeshBuffers)
		{
			constexpr bool bCPUAccess = true;

			bool bNewMesh = true;
			if (StaticMesh->GetRenderData())
			{
				bNewMesh = false;
				StaticMesh->ReleaseResources();
				StaticMesh->ReleaseResourcesFence.Wait();
			}
			
			StaticMesh->SetRenderData(MakeUnique<FStaticMeshRenderData>());
			FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
			RenderData->AllocateLODResources(1);
			FStaticMeshLODResources& LODResources = RenderData->LODResources[0];

			TArray<FStaticMeshBuildVertex> StaticMeshBuildVertices;
			StaticMeshBuildVertices.SetNum(MeshBuffers->Vertices.Num());
			FStaticMeshBuildVertex* StaticMeshBuildVertexPtr = StaticMeshBuildVertices.GetData();

			for(const LidarPointCloudMeshing::FVertexData& Vertex : MeshBuffers->Vertices)
			{
				StaticMeshBuildVertexPtr->Position = Vertex.Position;
				StaticMeshBuildVertexPtr->Color = Vertex.Color;
				StaticMeshBuildVertexPtr->TangentZ = Vertex.Normal;
				StaticMeshBuildVertexPtr++;
			}

			LODResources.bBuffersInlined = true;
			LODResources.IndexBuffer.TrySetAllowCPUAccess(bCPUAccess);
			LODResources.VertexBuffers.PositionVertexBuffer.Init(StaticMeshBuildVertices);
			LODResources.VertexBuffers.StaticMeshVertexBuffer.Init(StaticMeshBuildVertices, 1, bCPUAccess);
			LODResources.bHasColorVertexData = true;
			LODResources.VertexBuffers.ColorVertexBuffer.Init(StaticMeshBuildVertices);
			LODResources.bHasReversedIndices = false;
			LODResources.bHasReversedDepthOnlyIndices = false;
			
			FStaticMeshSection& Section = LODResources.Sections.AddDefaulted_GetRef();
			Section.FirstIndex = 0;
			Section.NumTriangles = MeshBuffers->Indices.Num() / 3;
			Section.MinVertexIndex = 0;
			Section.MaxVertexIndex = MeshBuffers->Vertices.Num() - 1;			
			Section.MaterialIndex = 0;
			Section.bEnableCollision = true;
			Section.bCastShadow = true;			
			LODResources.IndexBuffer.SetIndices(MeshBuffers->Indices, EIndexBufferStride::Force32Bit);
				
			LODResources.bHasDepthOnlyIndices = true;
			LODResources.DepthOnlyIndexBuffer.SetIndices(MeshBuffers->Indices, EIndexBufferStride::Force32Bit);
			LODResources.DepthOnlyNumTriangles = MeshBuffers->Indices.Num() / 3;

			StaticMesh->InitResources();

			// Set up RenderData bounds and LOD data
			RenderData->Bounds = MeshBuffers->Bounds;
			StaticMesh->CalculateExtendedBounds();
				
			RenderData->ScreenSize[0].Default = 1.0f;

			// Set up physics-related data
			StaticMesh->CreateBodySetup();
			check(StaticMesh->GetBodySetup());
			StaticMesh->GetBodySetup()->InvalidatePhysicsData();

			if (!bNewMesh)
			{
				for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
				{
					UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
					if (StaticMeshComponent->GetStaticMesh() == StaticMesh)
					{
						// it needs to recreate IF it already has been created
						if (StaticMeshComponent->IsPhysicsStateCreated())
						{
							StaticMeshComponent->RecreatePhysicsState();
						}
					}
				}
			}
			
			TArray<FStaticMaterial> StaticMaterials;
			static UMaterialInterface* Material = Cast<UMaterialInterface>(FSoftObjectPath(TEXT("/LidarPointCloud/Materials/M_MeshedCloud.M_MeshedCloud")).TryLoad());
			StaticMaterials.Emplace(Material);
			StaticMesh->SetStaticMaterials(StaticMaterials);

			StaticMesh->SetLightingGuid(FGuid::NewGuid()) ;
			StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
			StaticMesh->PostEditChange();
			StaticMesh->MarkPackageDirty();
		}
	}
	
	TSharedPtr<FSceneView> GetEditorView(FEditorViewportClient* ViewportClient)
	{
		FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport,
			ViewportClient->GetScene(),
			ViewportClient->EngineShowFlags).SetRealtimeUpdate(ViewportClient->IsRealtime()));

		return MakeShareable<FSceneView>(ViewportClient->CalcSceneView(&ViewFamily));
	}

	FConvexVolume BuildConvexVolumeForPoints(const TArray<FVector2D>& Points, FEditorViewportClient* ViewportClient)
	{
		if(!ViewportClient)
		{
			ViewportClient = GCurrentLevelEditingViewportClient;
		}
		
		FConvexVolume ConvexVolume;

		const TSharedPtr<const FSceneView> View = GetEditorView(ViewportClient);
		const FMatrix InvViewProjectionMatrix = View->ViewMatrices.GetClipToWorld();

		TArray<FVector> Origins; Origins.AddUninitialized(Points.Num() + 2);
		TArray<FVector> Normals; Normals.AddUninitialized(Points.Num() + 2);
		TArray<FVector> Directions; Directions.AddUninitialized(Points.Num());
		FVector MeanCenter = FVector::ZeroVector;

		for (int32 i = 0; i < Points.Num(); ++i)
		{
			FSceneView::DeprojectScreenToWorld(Points[i], FIntRect(FIntPoint(0, 0), ViewportClient->Viewport->GetSizeXY()), InvViewProjectionMatrix, Origins[i], Directions[i]);
			MeanCenter += Origins[i];
		}

		MeanCenter /= Points.Num();

		const FVector& ViewDirection = View->GetViewDirection();

		// Shared calculations
		Normals.Last(1) = ViewDirection;
		Normals.Last() = -ViewDirection;
		Origins.Last(1) = Origins[0] + ViewDirection * 99999999.0f;

		// Calculate plane normals
		for (int32 i = 0; i < Points.Num(); ++i)
		{
			Normals[i] = ((Origins[(i + 1) % Points.Num()] - Origins[i]).GetSafeNormal() ^ Directions[i]).GetSafeNormal();
		}

		// Flip normals?
		if(FVector::DotProduct(Normals[0], (MeanCenter - Origins[0])) > 0)
		{
			for (int32 i = 0; i < Points.Num(); ++i)
			{
				Normals[i] = -Normals[i];
			}
		}

		// Perspective View
		if (View->IsPerspectiveProjection())
		{
			Origins.Last() = Origins[0];
		}
		// Ortho Views
		else
		{
			Origins.Last() = -Origins.Last(1);
		}

		for (int32 i = 0; i < Origins.Num(); ++i)
		{
			ConvexVolume.Planes.Emplace(Origins[i], Normals[i]);
		}

		ConvexVolume.Init();

		return ConvexVolume;
	}
	
	ULidarPointCloud* Extract_Internal()
	{
		ULidarPointCloud* NewPointCloud = CreateNewAsset_Internal();
		if (NewPointCloud)
		{
			int64 NumPoints = 0;
			ProcessAll([&NumPoints](ALidarPointCloudActor* Actor)
			{
				NumPoints += Actor->GetPointCloudComponent()->NumSelectedPoints();
			});

			TArray64<FLidarPointCloudPoint> SelectedPoints;
			SelectedPoints.Reserve(NumPoints);
			ProcessAll([&SelectedPoints](ALidarPointCloudActor* Actor)
			{
				Actor->GetPointCloudComponent()->GetSelectedPointsAsCopies(SelectedPoints, true);
			});
			
			NewPointCloud->SetData(SelectedPoints);
		}
		return NewPointCloud;
	}
	
	// Copied from GeomTools.cpp
	bool IsPolygonConvex(const TArray<FVector2D>& Points)
	{
		const int PointCount = Points.Num();
		float Sign = 0;
		for (int32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
		{
			const FVector2D& A = Points[PointIndex];
			const FVector2D& B = Points[(PointIndex + 1) % PointCount];
			const FVector2D& C = Points[(PointIndex + 2) % PointCount];
			float Det = (B.X - A.X) * (C.Y - B.Y) - (B.Y - A.Y) * (C.X - B.X);
			float DetSign = FMath::Sign(Det);
			if (DetSign != 0)
			{
				if (Sign == 0)
				{
					Sign = DetSign;
				}
				else if (Sign != DetSign)
				{
					return false;
				}
			}
		}

		return true;
	}
}

ULidarPointCloud* FLidarPointCloudEditorHelper::CreateNewAsset()
{
	return CreateNewAsset_Internal();
}

void FLidarPointCloudEditorHelper::AlignSelectionAroundWorldOrigin()
{
	ULidarPointCloud::AlignClouds(GetSelectedClouds());
}

void FLidarPointCloudEditorHelper::SetOriginalCoordinateForSelection()
{
	ProcessSelection([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->RestoreOriginalCoordinates();
		}
	});
}

void FLidarPointCloudEditorHelper::CenterSelection()
{
	ProcessSelection([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->CenterPoints();
		}
	});
}

void FLidarPointCloudEditorHelper::BuildCollisionForSelection()
{
	ProcessSelection([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->BuildCollision();
		}
	});
}

void FLidarPointCloudEditorHelper::SetCollisionErrorForSelection(float Error)
{
	ProcessSelection([Error](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			if(Error > 0)
			{
				PointCloud->MaxCollisionError = Error;
			}
			else
			{
				PointCloud->SetOptimalCollisionError();
			}
		}
	});
}

void FLidarPointCloudEditorHelper::RemoveCollisionForSelection()
{
	ProcessSelection([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->RemoveCollision();
		}
	});
}

void FLidarPointCloudEditorHelper::MeshSelected(bool bMeshByPoints, float CellSize, bool bMergeMeshes, bool bRetainTransform)
{
	const int32 NumSteps = (bMeshByPoints ? GetNumLidarActors() : GEditor->GetSelectedActorCount()) + (bMergeMeshes ? 1 : 0);
	
	FScopedSlowTask ProgressDialog(NumSteps, LOCTEXT("Meshing", "Meshing Point Clouds..."));
	ProgressDialog.MakeDialog();
	
	if(bMergeMeshes)
	{
		if(UStaticMesh* StaticMesh = CreateStaticMesh())
		{
			FScopeBenchmarkTimer Timer("MakeMesh");
			LidarPointCloudMeshing::FMeshBuffers MeshBuffers;

			TFunction<void(ALidarPointCloudActor*)> MergeFunc = [CellSize, &ProgressDialog, MeshBuffersPtr = &MeshBuffers, bMeshByPoints](ALidarPointCloudActor* Actor)
			{
				if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
				{
					if(bMeshByPoints)
					{
						PointCloud->BuildStaticMeshBuffersForSelection(CellSize, MeshBuffersPtr, Actor->GetActorTransform());
					}
					else
					{
						PointCloud->BuildStaticMeshBuffers(CellSize, MeshBuffersPtr, Actor->GetActorTransform());
					}
				}
					
				ProgressDialog.EnterProgressFrame(1.0f);
			};

			if(bMeshByPoints)
			{
				ProcessAll(MoveTemp(MergeFunc));
			}
			else
			{
				ProcessSelection(MoveTemp(MergeFunc));
			}

			FMeshDescription MeshDescription;
			MeshBuffersToMeshDescription(MeshBuffers, MeshDescription);
			AssignMeshDescriptionToMesh(MeshDescription, StaticMesh);

			ProgressDialog.EnterProgressFrame(1.0f);
					
			if(AStaticMeshActor* MeshActor = SpawnActor<AStaticMeshActor>(StaticMesh->GetName()))
			{
				MeshActor->GetStaticMeshComponent()->SetStaticMesh(StaticMesh);
			}
		}
	}
	else
	{
		TFunction<void(ALidarPointCloudActor*)> MergeFunc = [CellSize, &ProgressDialog, bRetainTransform, bMeshByPoints](ALidarPointCloudActor* Actor)
		{
			if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
			{
				if(UStaticMesh* StaticMesh = CreateStaticMesh())
				{
					LidarPointCloudMeshing::FMeshBuffers MeshBuffers;
					
					if(bMeshByPoints)
					{
						PointCloud->BuildStaticMeshBuffersForSelection(CellSize, &MeshBuffers, bRetainTransform ? FTransform::Identity : Actor->GetActorTransform());
					}
					else
					{
						PointCloud->BuildStaticMeshBuffers(CellSize, &MeshBuffers, bRetainTransform ? FTransform::Identity : Actor->GetActorTransform());
					}
					
					FMeshDescription MeshDescription;
					MeshBuffersToMeshDescription(MeshBuffers, MeshDescription);
					AssignMeshDescriptionToMesh(MeshDescription, StaticMesh);
					
					if(AStaticMeshActor* MeshActor = SpawnActor<AStaticMeshActor>(StaticMesh->GetName()))
					{
						if(bRetainTransform)
						{
							MeshActor->SetActorTransform(Actor->GetActorTransform());
						}
					 	
						MeshActor->GetStaticMeshComponent()->SetStaticMesh(StaticMesh);
					}
				}
			}
				
			ProgressDialog.EnterProgressFrame(1.0f);
		};
		
		if(bMeshByPoints)
		{
			ProcessAll(MoveTemp(MergeFunc));
		}
		else
		{
			ProcessSelection(MoveTemp(MergeFunc));
		}
	}
}

void FLidarPointCloudEditorHelper::CalculateNormalsForSelection()
{
	ProcessSelection([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->CalculateNormalsForSelection();
		}
	});
}

void FLidarPointCloudEditorHelper::SetNormalsQuality(int32 Quality, float NoiseTolerance)
{
	ProcessAll([Quality, NoiseTolerance](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->NormalsQuality = Quality;
			PointCloud->NormalsNoiseTolerance = NoiseTolerance;
		}
	});
}

void FLidarPointCloudEditorHelper::ResetVisibility()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->UnhideAll();
		}
	});
}

void FLidarPointCloudEditorHelper::DeleteHidden()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->RemoveHiddenPoints();
		}
	});
}

void FLidarPointCloudEditorHelper::Extract()
{
	if(ULidarPointCloud* NewPointCloud = Extract_Internal())
	{
		if(ALidarPointCloudActor* Actor = SpawnActor<ALidarPointCloudActor>(NewPointCloud->GetName()))
		{
			NewPointCloud->RestoreOriginalCoordinates();
			Actor->SetPointCloud(NewPointCloud);
		}

		DeleteSelected();
	}
}

void FLidarPointCloudEditorHelper::ExtractAsCopy()
{
	if(ULidarPointCloud* NewPointCloud = Extract_Internal())
	{
		if(ALidarPointCloudActor* Actor = SpawnActor<ALidarPointCloudActor>(NewPointCloud->GetName()))
		{
			NewPointCloud->RestoreOriginalCoordinates();
			Actor->SetPointCloud(NewPointCloud);
		}
	
		ClearSelection();
	}
}

void FLidarPointCloudEditorHelper::CalculateNormals()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloud* PointCloud = Actor->GetPointCloud())
		{
			PointCloud->CalculateNormals(nullptr, nullptr);
		}
	});
}

FConvexVolume FLidarPointCloudEditorHelper::BuildConvexVolumeFromCoordinates(FVector2d Start, FVector2d End, FEditorViewportClient* ViewportClient)
{
	FIntVector4 SelectionArea;
	SelectionArea.X = FMath::Min(Start.X, End.X);
	SelectionArea.Y = FMath::Min(Start.Y, End.Y);
	SelectionArea.Z = FMath::Max(Start.X, End.X);
	SelectionArea.W = FMath::Max(Start.Y, End.Y);

	return BuildConvexVolumeForPoints(TArray<FVector2D>({
			FVector2D(SelectionArea.X, SelectionArea.Y),
			FVector2D(SelectionArea.X, SelectionArea.W),
			FVector2D(SelectionArea.Z, SelectionArea.W),
			FVector2D(SelectionArea.Z, SelectionArea.Y) }), ViewportClient);
}

TArray<FConvexVolume> FLidarPointCloudEditorHelper::BuildConvexVolumesFromPoints(TArray<FVector2d> Points, FEditorViewportClient* ViewportClient)
{
	TArray<FConvexVolume> ConvexVolumes;
		
	if (IsPolygonConvex(Points))
	{
		ConvexVolumes.Add(BuildConvexVolumeForPoints(Points, ViewportClient));
	}
	else
	{
		// Check for self-intersecting shape
		if (!IsPolygonSelfIntersecting(Points, true))
		{
			TArray<TArray<FVector2D>> ConvexShapes;
			
			// The separation needs points in CCW order
			if (!FGeomTools2D::IsPolygonWindingCCW(Points))
			{
				Algo::Reverse(Points);
			}

			TArray<FVector2D> Triangles;
			FGeomTools2D::TriangulatePoly(Triangles, Points, false);
			FGeomTools2D::GenerateConvexPolygonsFromTriangles(ConvexShapes, Triangles);

			for (int32 i = 0; i < ConvexShapes.Num(); ++i)
			{
				ConvexVolumes.Add(BuildConvexVolumeForPoints(ConvexShapes[i], ViewportClient));
			}
		}
	}

	return ConvexVolumes;
}

FLidarPointCloudRay FLidarPointCloudEditorHelper::MakeRayFromScreenPosition(FVector2d Position, FEditorViewportClient* ViewportClient)
{
	if(!ViewportClient)
	{
		ViewportClient = GCurrentLevelEditingViewportClient;
	}
	
	const TSharedPtr<const FSceneView> View = GetEditorView(ViewportClient);
	const FMatrix InvViewProjectionMatrix = View->ViewMatrices.GetClipToWorld();

	FVector3d Origin, Direction;
	
	FSceneView::DeprojectScreenToWorld(Position, FIntRect(FIntPoint(0, 0), ViewportClient->Viewport->GetSizeXY()), InvViewProjectionMatrix, Origin, Direction);

	return FLidarPointCloudRay(Origin, Direction);
}

bool FLidarPointCloudEditorHelper::RayTracePointClouds(const FLidarPointCloudRay& Ray, float RadiusMulti, FVector3f& OutHitLocation)
{
	float MinDistance = FLT_MAX;
	
	ProcessAll([Ray, &MinDistance, &OutHitLocation, RadiusMulti](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloudComponent* Component = Actor->GetPointCloudComponent())
		{
			if(const ULidarPointCloud* PointCloud = Component->GetPointCloud())
			{
				const float TraceRadius = FMath::Max(PointCloud->GetEstimatedPointSpacing(), 0.5f) * RadiusMulti;
				if(const FLidarPointCloudPoint* Point = Component->LineTraceSingle(Ray, TraceRadius, true))
				{
					const FVector3f PointLocation = (FVector3f)(Component->GetComponentTransform().TransformPosition((FVector)Point->Location) + PointCloud->LocationOffset);
					
					const float DistanceSq = (Ray.Origin - PointLocation).SizeSquared();
					if(DistanceSq < MinDistance)
					{
						MinDistance = DistanceSq;
						OutHitLocation = PointLocation;
					}
				}
			}
		}
	});

	return MinDistance < FLT_MAX;
}

bool FLidarPointCloudEditorHelper::IsPolygonSelfIntersecting(const TArray<FVector2D>& Points, bool bAllowLooping)
{
	// Slow, O(n2), but sufficient for the current problem
	
	const int32 MaxIndex = bAllowLooping ? Points.Num() : Points.Num() - 1;

	for (int32 i = 0; i < MaxIndex; ++i)
	{
		const int32 i1 = (i + 1) % Points.Num();

		const FVector2D P1 = Points[i];
		const FVector2D P2 = Points[i1];

		for (int32 j = 0; j < MaxIndex; ++j)
		{
			const int32 j1 = j < Points.Num() - 1 ? j + 1 : 0;

			if (j1 != i && j != i && j != i1)
			{
				// Modified FMath::SegmentIntersection2D
				// Inlining the code and skipping calculation of an intersection point makes a slight difference for O(n2)
				const FVector2D SegmentStartA = P1;
				const FVector2D SegmentEndA = P2;
				const FVector2D SegmentStartB = Points[j];
				const FVector2D SegmentEndB = Points[j1];
				const FVector2D VectorA = P2 - SegmentStartA;
				const FVector2D VectorB = SegmentEndB - SegmentStartB;

				const float S = (-VectorA.Y * (SegmentStartA.X - SegmentStartB.X) + VectorA.X * (SegmentStartA.Y - SegmentStartB.Y)) / (-VectorB.X * VectorA.Y + VectorA.X * VectorB.Y);
				if (S >= 0 && S <= 1)
				{
					const float T = (VectorB.X * (SegmentStartA.Y - SegmentStartB.Y) - VectorB.Y * (SegmentStartA.X - SegmentStartB.X)) / (-VectorB.X * VectorA.Y + VectorA.X * VectorB.Y);
					if (T >= 0 && T <= 1)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

bool FLidarPointCloudEditorHelper::AreLidarActorsSelected()
{
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if(Cast<ALidarPointCloudActor>(*It))
		{
			return true;
		}
	}

	return false;
}

bool FLidarPointCloudEditorHelper::AreLidarPointsSelected()
{
	for (TObjectIterator<ALidarPointCloudActor> It; It; ++It)
	{
		if(const ALidarPointCloudActor* Actor = Cast<ALidarPointCloudActor>(*It))
		{
			if(const ULidarPointCloud* PointCloud = Actor->GetPointCloud())
			{
				if(PointCloud->HasSelectedPoints())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FLidarPointCloudEditorHelper::SelectPointsByConvexVolume(const FConvexVolume& ConvexVolume, ELidarPointCloudSelectionMode SelectionMode)
{
	ProcessAll([ConvexVolume, SelectionMode](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloudComponent* Component = Actor->GetPointCloudComponent())
		{
			Component->SelectByConvexVolume(ConvexVolume, SelectionMode != ELidarPointCloudSelectionMode::Subtract, true);
		}
	});
}

void FLidarPointCloudEditorHelper::SelectPointsBySphere(FSphere Sphere, ELidarPointCloudSelectionMode SelectionMode)
{
	ProcessAll([Sphere, SelectionMode](ALidarPointCloudActor* Actor)
	{
		if(ULidarPointCloudComponent* Component = Actor->GetPointCloudComponent())
		{
			Component->SelectBySphere(Sphere, SelectionMode != ELidarPointCloudSelectionMode::Subtract, true);
		}
	});
}

void FLidarPointCloudEditorHelper::HideSelected()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		Actor->GetPointCloudComponent()->HideSelected();
	});
}

void FLidarPointCloudEditorHelper::DeleteSelected()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		Actor->GetPointCloudComponent()->DeleteSelected();
	});
}

void FLidarPointCloudEditorHelper::InvertSelection()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		Actor->GetPointCloudComponent()->InvertSelection();
	});
}

void FLidarPointCloudEditorHelper::ClearSelection()
{
	ProcessAll([](ALidarPointCloudActor* Actor)
	{
		Actor->GetPointCloudComponent()->ClearSelection();
	});
}

void FLidarPointCloudEditorHelper::InvertActorSelection()
{
	TArray<ALidarPointCloudActor*> SelectedActors = GetSelectedActors();

	ProcessAll([&SelectedActors](ALidarPointCloudActor* Actor)
	{
		GEditor->SelectActor(Actor, !SelectedActors.Contains(Actor), true);
	});
}

void FLidarPointCloudEditorHelper::ClearActorSelection()
{
	GEditor->SelectNone(true, true);
}

void FLidarPointCloudEditorHelper::MergeLidar(ULidarPointCloud* TargetAsset, TArray<ULidarPointCloud*> SourceAssets)
{
	if(!IsValid(TargetAsset) || SourceAssets.Num() == 0)
	{
		return;
	}
	
	FScopedSlowTask ProgressDialog(SourceAssets.Num() + 2, LOCTEXT("Merge", "Merging Point Clouds..."));
	ProgressDialog.MakeDialog();

	TargetAsset->Merge(SourceAssets, [&ProgressDialog]() { ProgressDialog.EnterProgressFrame(1.f); });

	FAssetRegistryModule::AssetCreated(TargetAsset);
	TargetAsset->MarkPackageDirty();
}

void FLidarPointCloudEditorHelper::MergeSelectionByData(bool bReplaceSource)
{
	TArray<ALidarPointCloudActor*> Actors = GetSelectedActors();
	TArray<ULidarPointCloud*> PointClouds = GetSelectedClouds();
	
	if (PointClouds.Num() > 1)
	{
		ULidarPointCloud* NewCloud = CreateNewAsset();
		
		MergeLidar(NewCloud, PointClouds);

		if(bReplaceSource)
		{
			// Repurpose the first actor
			Actors[0]->SetPointCloud(NewCloud);
			
			// Remove the rest
			for(int32 i = 1; i < Actors.Num(); ++i)
			{
				Actors[i]->Destroy();
			}
		}
		else
		{
			ALidarPointCloudActor* NewActor = Actors[0]->GetWorld()->SpawnActor<ALidarPointCloudActor>(Actors[0]->GetActorLocation(), Actors[0]->GetActorRotation());
			NewActor->SetPointCloud(NewCloud);
		}
	}
}

void FLidarPointCloudEditorHelper::MergeSelectionByComponent(bool bReplaceSource)
{
	TArray<ALidarPointCloudActor*> Actors = GetSelectedActors();

	if (Actors.Num() > 1)
	{
		AActor* TargetActor = Actors[0]->GetWorld()->SpawnActor<ALidarPointCloudActor>();
		
		for(ALidarPointCloudActor* Actor : Actors)
		{
			TArray<ULidarPointCloudComponent*> Components;
			Actor->GetComponents(Components);

			for(const ULidarPointCloudComponent* Component : Components)
			{
				ULidarPointCloudComponent* PCC = (ULidarPointCloudComponent*)TargetActor->AddComponentByClass(ULidarPointCloudComponent::StaticClass(), true, Component->GetComponentTransform(), false);
				PCC->SetPointCloud(Component->GetPointCloud());
				PCC->SetWorldTransform(Component->GetComponentTransform());
				PCC->AttachToComponent(TargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepWorld, false));
			}

			if(bReplaceSource)
			{
				Actor->Destroy();
			}
			else
			{
				Actor->SetHidden(true);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
