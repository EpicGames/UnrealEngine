// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"

class ALidarPointCloudActor;
class FEditorViewportClient;
class ULidarPointCloud;
struct FConvexVolume;

enum class ELidarPointCloudSelectionMode : uint8
{
	None,
	Add,
	Subtract
};

class FLidarPointCloudEditorHelper
{
public:
	static ULidarPointCloud* CreateNewAsset();

	static void AlignSelectionAroundWorldOrigin();
	static void SetOriginalCoordinateForSelection();
	static void CenterSelection();
	
	static void MergeLidar(ULidarPointCloud* TargetAsset, TArray<ULidarPointCloud*> SourceAssets);
	static void MergeSelectionByData(bool bReplaceSource);
	static void MergeSelectionByComponent(bool bReplaceSource);

	static void BuildCollisionForSelection();
	static void SetCollisionErrorForSelection(float Error);
	static void RemoveCollisionForSelection();

	static void MeshSelected(bool bMeshByPoints, float CellSize, bool bMergeMeshes, bool bRetainTransform);
	
	static void CalculateNormalsForSelection();
	static void SetNormalsQuality(int32 Quality, float NoiseTolerance);
	
	static void SelectPointsByConvexVolume(const FConvexVolume& ConvexVolume, ELidarPointCloudSelectionMode SelectionMode);
	static void SelectPointsBySphere(FSphere Sphere, ELidarPointCloudSelectionMode SelectionMode);
	static void HideSelected();
	static void DeleteSelected();
	static void InvertSelection();
	static void ClearSelection();
	
	static void InvertActorSelection();
	static void ClearActorSelection();

	static void ResetVisibility();
	static void DeleteHidden();
	static void Extract();
	static void ExtractAsCopy();
	static void CalculateNormals();

	static FConvexVolume BuildConvexVolumeFromCoordinates(FVector2d Start, FVector2d End, FEditorViewportClient* ViewportClient = nullptr);
	static TArray<FConvexVolume> BuildConvexVolumesFromPoints(TArray<FVector2d> Points, FEditorViewportClient* ViewportClient = nullptr);
	static FLidarPointCloudRay MakeRayFromScreenPosition(FVector2d Position, FEditorViewportClient* ViewportClient = nullptr);

	static bool RayTracePointClouds(const FLidarPointCloudRay& Ray, float RadiusMulti, FVector3f& OutHitLocation);

	static bool IsPolygonSelfIntersecting(const TArray<FVector2D>& Points, bool bAllowLooping);

	static bool AreLidarActorsSelected();
	static bool AreLidarPointsSelected();
};
