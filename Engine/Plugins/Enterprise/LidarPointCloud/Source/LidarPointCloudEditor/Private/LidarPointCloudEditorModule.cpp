// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModeRegistry.h"
#include "ILidarPointCloudEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "LevelEditor.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudFactory.h"
#include "LidarPointCloudStyle.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudSettings.h"
#include "ISettingsModule.h"
#include "LidarPointCloudEditorCommands.h"
#include "LidarPointCloudEdMode.h"
#include "ShowFlags.h"

DEFINE_LOG_CATEGORY(LogLidarPointCloud);

#define LOCTEXT_NAMESPACE "LidarPointCloudEditor"

class FLidarPointCloudEditorModule : public ILidarPointCloudEditorModule
{
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	// Begin IModuleInterface Interface
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		// Register slate style overrides
		FLidarPointCloudStyle::Initialize();
		
		// Register settings
		if (ISettingsModule * SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "LidarPointCloud",
				LOCTEXT("SettingsName", "LiDAR Point Cloud"),
				LOCTEXT("SettingsDescription", "Configure the LiDAR Point Cloud plugin"),
				GetMutableDefault<ULidarPointCloudSettings>());
		}

		FEngineShowFlags::RegisterCustomShowFlag(TEXT("LidarClippingVolume"), true, EShowFlagGroup::SFG_Transient, NSLOCTEXT("UnrealEd", "LidarClippingVolumeSF", "Lidar Clipping Volume"));

		FLidarPointCloudEditorCommands::Register();
	}
	virtual void ShutdownModule() override
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		FLidarPointCloudEditorCommands::Unregister();

		FEditorModeRegistry::Get().UnregisterMode(FLidarEditorModes::EM_Lidar);

		if (UObjectInitialized())
		{
			// Unregister settings
			if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
			{
				SettingsModule->UnregisterSettings("Project", "Plugins", "LidarPointCloud");
			}
		}

		// Unregister slate style overrides
		FLidarPointCloudStyle::Shutdown();
	}
	// End IModuleInterface Interface
	
private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};

IMPLEMENT_MODULE(FLidarPointCloudEditorModule, LidarPointCloudEditor)

#undef LOCTEXT_NAMESPACE
