// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEditorTools.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "ConvexVolume.h"
#include "LidarPointCloudEditorHelper.h"
#include "LidarPointCloudShared.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "PrimitiveDrawingUtils.h"

#define LOCTEXT_NAMESPACE "LidarPointCloudEditorTool"

namespace {
	// Distance Square between the first and last points of the polygonal selection, where the shape will be considered as closed
	constexpr int32 PolySnapDistanceSq = 250;

	// Affects the frequency of new point injections when drawing lasso-based shapes
	constexpr int32 LassoSpacingSq = 250;

	// Affects the max depth delta when painting. Prevents the brush from "falling through" the gaps.
	constexpr float PaintMaxDeviation = 0.15f;
}

void ULidarEditorToolBase::Setup()
{
	Super::Setup();

	ToolActions = CreateToolActions();

	if(ToolActions)
	{
		AddToolPropertySource(ToolActions);
	}
}

FText ULidarEditorToolBase::GetToolMessage() const
{
	return FText();
}

void ULidarEditorToolClickDragBase::Setup()
{
	Super::Setup();

	HoverBehavior = NewObject<UMouseHoverBehavior>();	
	HoverBehavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	HoverBehavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior, this);
	
	ClickDragBehavior = NewObject<UClickDragInputBehavior>();
	ClickDragBehavior->Initialize(this);
	ClickDragBehavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	ClickDragBehavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	AddInputBehavior(ClickDragBehavior, this);
}

void ULidarEditorToolClickDragBase::Render(IToolsContextRenderAPI* RenderAPI)
{
	Super::Render(RenderAPI);
	
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
}

void ULidarEditorToolClickDragBase::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	switch (ModifierID)
	{
	case 1:
		bShiftToggle = bIsOn;
		break;
	case 2:
		bCtrlToggle = bIsOn;
		break;
	default:
		break;
	}
}

void ULidarToolActionsAlign::AlignAroundWorldOrigin()
{
	FLidarPointCloudEditorHelper::AlignSelectionAroundWorldOrigin();
}

void ULidarToolActionsAlign::AlignAroundOriginalCoordinates()
{
	FLidarPointCloudEditorHelper::SetOriginalCoordinateForSelection();
}

void ULidarToolActionsAlign::ResetAlignment()
{
	FLidarPointCloudEditorHelper::CenterSelection();
}

void ULidarToolActionsMerge::MergeActors()
{
	FLidarPointCloudEditorHelper::MergeSelectionByComponent(bReplaceSourceActorsAfterMerging);
}

void ULidarToolActionsMerge::MergeData()
{
	FLidarPointCloudEditorHelper::MergeSelectionByData(bReplaceSourceActorsAfterMerging);
}

void ULidarToolActionsCollision::BuildCollision()
 {
 	FLidarPointCloudEditorHelper::SetCollisionErrorForSelection(OverrideMaxCollisionError);
 	FLidarPointCloudEditorHelper::BuildCollisionForSelection();
 }

void ULidarToolActionsCollision::RemoveCollision()
 {
 	FLidarPointCloudEditorHelper::RemoveCollisionForSelection();
 }

void ULidarToolActionsMeshing::BuildStaticMesh()
{
	FLidarPointCloudEditorHelper::MeshSelected(false, MaxMeshingError, bMergeMeshes, !bMergeMeshes && bRetainTransform);
}

void ULidarToolActionsNormals::CalculateNormals()
{
	FLidarPointCloudEditorHelper::SetNormalsQuality(Quality, NoiseTolerance);
	FLidarPointCloudEditorHelper::CalculateNormalsForSelection();
}

void ULidarToolActionsSelection::HideSelected()
{
	FLidarPointCloudEditorHelper::HideSelected();
}

void ULidarToolActionsSelection::ResetVisibility()
{
	FLidarPointCloudEditorHelper::ResetVisibility();
}

void ULidarToolActionsSelection::DeleteHidden()
{
	FLidarPointCloudEditorHelper::DeleteHidden();
}

void ULidarToolActionsSelection::Extract()
{
	FLidarPointCloudEditorHelper::Extract();
}

void ULidarToolActionsSelection::ExtractAsCopy()
{
	FLidarPointCloudEditorHelper::ExtractAsCopy();
}

void ULidarToolActionsSelection::CalculateNormals()
{
	FLidarPointCloudEditorHelper::CalculateNormals();
	ClearSelection();
}

void ULidarToolActionsSelection::DeleteSelected()
{
	FLidarPointCloudEditorHelper::DeleteSelected();
}

void ULidarToolActionsSelection::InvertSelection()
{
	FLidarPointCloudEditorHelper::InvertSelection();
}

void ULidarToolActionsSelection::ClearSelection()
{
	FLidarPointCloudEditorHelper::ClearSelection();
}

void ULidarToolActionsSelection::BuildStaticMesh()
{
	FLidarPointCloudEditorHelper::MeshSelected(true, MaxMeshingError, bMergeMeshes, !bMergeMeshes && bRetainTransform);
}

void ULidarEditorToolSelectionBase::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if(Clicks.Num() == 0)
	{
		return;
	}

	const FLinearColor Color = GetHUDColor();
	
	for(int32 i = 1; i < Clicks.Num(); ++i)
	{
		FCanvasLineItem LineItem(Clicks[i - 1] / Canvas->GetDPIScale(), Clicks[i] / Canvas->GetDPIScale());
		LineItem.SetColor(Color);
		Canvas->DrawItem(LineItem);
	}
	
	FCanvasLineItem LineItem(CurrentMousePos / Canvas->GetDPIScale(), Clicks.Last() / Canvas->GetDPIScale());
	LineItem.SetColor(Color);
	Canvas->DrawItem(LineItem);
}

TArray<FConvexVolume> ULidarEditorToolSelectionBase::GetSelectionConvexVolumes()
{
	return FLidarPointCloudEditorHelper::BuildConvexVolumesFromPoints(Clicks);
}

TObjectPtr<UInteractiveToolPropertySet> ULidarEditorToolSelectionBase::CreateToolActions()
{
	return NewObject<ULidarToolActionsSelection>(this);
}

bool ULidarEditorToolSelectionBase::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	CurrentMousePos = DevicePos.ScreenPosition;
	PostCurrentMousePosChanged();
	return true;
}

void ULidarEditorToolSelectionBase::OnClickPress(const FInputDeviceRay& PressPos)
{
	bSelecting = true;
}

void ULidarEditorToolSelectionBase::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	bSelecting = false;
}

void ULidarEditorToolSelectionBase::OnClickDrag(const FInputDeviceRay& DragPos)
{
	CurrentMousePos = DragPos.ScreenPosition;
	PostCurrentMousePosChanged();
}

void ULidarEditorToolSelectionBase::OnTerminateDragSequence()
{
	Clicks.Empty();
	bSelecting = false;
}

bool ULidarEditorToolSelectionBase::ExecuteNestedCancelCommand()
{
	OnTerminateDragSequence();
	return true;
}

FText ULidarEditorToolSelectionBase::GetToolMessage() const
{
	const FText ToolMessage = LOCTEXT("ULidarEditorToolToolMessage", "Use Left-click to start the selection. Hold Shift to add selection, hold Ctrl to subtract selection.");
	return ToolMessage;
}

FLinearColor ULidarEditorToolSelectionBase::GetHUDColor()
{
	return FLinearColor::White;
}

void ULidarEditorToolSelectionBase::FinalizeSelection()
{
	const ELidarPointCloudSelectionMode SelectionMode = GetSelectionMode();

	if(SelectionMode == ELidarPointCloudSelectionMode::None)
	{
		FLidarPointCloudEditorHelper::ClearSelection();
	}
	
	const TArray<FConvexVolume> ConvexVolumes = GetSelectionConvexVolumes();
	for (int32 i = 0; i < ConvexVolumes.Num(); ++i)
	{
		// Consecutive shapes need to be additive
		FLidarPointCloudEditorHelper::SelectPointsByConvexVolume(ConvexVolumes[i], i > 0 ? ELidarPointCloudSelectionMode::Add : SelectionMode);
	}
}

ELidarPointCloudSelectionMode ULidarEditorToolSelectionBase::GetSelectionMode() const
{
	ELidarPointCloudSelectionMode SelectionMode = ELidarPointCloudSelectionMode::None;
	if(bCtrlToggle)
	{
		SelectionMode = ELidarPointCloudSelectionMode::Subtract;
	}
	else if(bShiftToggle)
	{
		SelectionMode = ELidarPointCloudSelectionMode::Add;
	}
	return SelectionMode;
}

TArray<FConvexVolume> ULidarEditorToolBoxSelection::GetSelectionConvexVolumes()
{
	return { FLidarPointCloudEditorHelper::BuildConvexVolumeFromCoordinates(Clicks[0], Clicks[2]) };
}

void ULidarEditorToolBoxSelection::OnClickPress(const FInputDeviceRay& PressPos)
{
	Clicks.Append({
		PressPos.ScreenPosition,
		PressPos.ScreenPosition,
		PressPos.ScreenPosition,
		PressPos.ScreenPosition
	});
}

void ULidarEditorToolBoxSelection::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if(Clicks.Num() == 0)
	{
		return;
	}
	
	Clicks[1].Y = DragPos.ScreenPosition.Y;
	Clicks[2] = DragPos.ScreenPosition;
	Clicks[3].X = DragPos.ScreenPosition.X;
}

void ULidarEditorToolBoxSelection::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	if(Clicks.Num() == 4)
	{
		if(Clicks[0] == Clicks[2])
		{
			FLidarPointCloudEditorHelper::ClearSelection();
		}
		else
		{
			FinalizeSelection();
		}
	}
	
	Clicks.Empty();
}

void ULidarEditorToolPolygonalSelection::OnClickPress(const FInputDeviceRay& PressPos)
{
	if(IsWithinSnap())
	{
		FinalizeSelection();
		Clicks.Empty();
	}
	else
	{
		Clicks.Add(PressPos.ScreenPosition);
	}
}

FLinearColor ULidarEditorToolPolygonalSelection::GetHUDColor()
{
	return IsWithinSnap() ? FLinearColor::Green : Super::GetHUDColor();
}

void ULidarEditorToolPolygonalSelection::PostCurrentMousePosChanged()
{
	if(IsWithinSnap())
	{
		CurrentMousePos = Clicks[0];
	}
}

bool ULidarEditorToolPolygonalSelection::IsWithinSnap()
{
	return Clicks.Num() > 1 && (CurrentMousePos - Clicks[0]).SquaredLength() <= PolySnapDistanceSq;
}

void ULidarEditorToolLassoSelection::OnClickPress(const FInputDeviceRay& PressPos)
{
	Clicks.Add(PressPos.ScreenPosition);
}

void ULidarEditorToolLassoSelection::OnClickDrag(const FInputDeviceRay& DragPos)
{
	Super::OnClickDrag(DragPos);
	
	if(Clicks.Num() > 0 && (CurrentMousePos - Clicks.Last()).SquaredLength() >= LassoSpacingSq)
	{
		Clicks.Add(CurrentMousePos);
	}
}

void ULidarEditorToolLassoSelection::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	if(Clicks.Num() > 1)
	{
		FinalizeSelection();
	}

	Clicks.Empty();
}

void ULidarEditorToolPaintSelection::Setup()
{
	Super::Setup();
	BrushRadius = GetDefault<ULidarToolActionsPaintSelection>()->BrushRadius;
}

void ULidarEditorToolPaintSelection::Render(IToolsContextRenderAPI* RenderAPI)
{
	Super::Render(RenderAPI);
	
	if(bHasHit)
	{
		DrawWireSphere(RenderAPI->GetPrimitiveDrawInterface(), (FVector)HitLocation, FLinearColor::Red, BrushRadius, 32, SDPG_World);
	}
}

void ULidarEditorToolPaintSelection::PostCurrentMousePosChanged()
{
	const FLidarPointCloudRay Ray = FLidarPointCloudEditorHelper::MakeRayFromScreenPosition(CurrentMousePos);

	FVector3f NewHitLocation;
	bHasHit = FLidarPointCloudEditorHelper::RayTracePointClouds(Ray, 1, NewHitLocation);

	if(!bHasHit)
	{
		return;
	}
	
	const float NewDistance = FVector3f::Dist(NewHitLocation, Ray.Origin);
	const float Deviation = (NewDistance - LastHitDistance) / LastHitDistance;

	// If painting, prevent large depth changes
	// If not, query larger trace radius - if it passes the deviation test, it was a gap
	if (Deviation > PaintMaxDeviation && (bSelecting || (
						FLidarPointCloudEditorHelper::RayTracePointClouds(Ray, 6, NewHitLocation) &&
						(FVector3f::Dist(NewHitLocation, Ray.Origin) - LastHitDistance) / LastHitDistance <= PaintMaxDeviation)))
	{
		HitLocation = FVector3f(Ray.Origin + Ray.GetDirection() * LastHitDistance);
		return;
	}
	
	HitLocation = NewHitLocation;
	LastHitDistance = NewDistance;
}

void ULidarEditorToolPaintSelection::OnClickPress(const FInputDeviceRay& PressPos)
{
	Super::OnClickPress(PressPos);

	if(GetSelectionMode() == ELidarPointCloudSelectionMode::None)
	{
		FLidarPointCloudEditorHelper::ClearSelection();
	}
	
	Paint();
}

void ULidarEditorToolPaintSelection::OnClickDrag(const FInputDeviceRay& DragPos)
{
	Super::OnClickDrag(DragPos);
	Paint();
}

void ULidarEditorToolPaintSelection::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if(const ULidarToolActionsPaintSelection* Actions = Cast<ULidarToolActionsPaintSelection>(PropertySet))
	{
		if(Property && Property->GetName().Equals("BrushRadius"))
		{
			BrushRadius = Actions->BrushRadius;
		}
	}
}

void ULidarEditorToolPaintSelection::Paint()
{
	if(bHasHit)
	{
		FLidarPointCloudEditorHelper::SelectPointsBySphere(FSphere((FVector)HitLocation, BrushRadius), GetSelectionMode());
	}
}

#undef LOCTEXT_NAMESPACE
