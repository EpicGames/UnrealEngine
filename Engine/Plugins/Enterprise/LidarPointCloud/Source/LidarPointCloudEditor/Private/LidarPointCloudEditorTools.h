// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolQueryInterfaces.h"
#include "LidarPointCloudEditorHelper.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "LidarPointCloudEditorTools.generated.h"

class UMouseHoverBehavior;
class URectangleMarqueeMechanic;
class UClickDragInputBehavior;

UCLASS()
class ULidarEditorToolBase : public UInteractiveTool
{
public:
	GENERATED_BODY()
	virtual void Setup() override;
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() { return nullptr; }
	virtual FText GetToolMessage() const;
	virtual bool IsActorSelection() const { return true; }

	UPROPERTY()
	TObjectPtr<UInteractiveToolPropertySet> ToolActions = nullptr;
};

UCLASS()
class ULidarEditorToolBuilderBase : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override { return true; }
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolBase>(SceneState.ToolManager); }
};

UCLASS()
class ULidarEditorToolClickDragBase :
	public ULidarEditorToolBase,
	public IClickDragBehaviorTarget,
	public IHoverBehaviorTarget,
	public IInteractiveToolNestedAcceptCancelAPI
{
public:
	GENERATED_BODY()
	
	virtual void Setup() override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool IsActorSelection() const override { return false; }

	// IClickDragBehaviorTarget implementation
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override { return FInputRayHit(TNumericLimits<float>::Max()); }
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override {}
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override {}
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override {}
	virtual void OnTerminateDragSequence() override {}

	// IHoverBehaviorTarget implementation
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override { return FInputRayHit(TNumericLimits<float>::Max()); }
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override { return true; }
	virtual void OnEndHover() override {}
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	// IInteractiveToolNestedAcceptCancelAPI implementation
	virtual bool SupportsNestedCancelCommand() override { return true; }
	virtual bool CanCurrentlyNestedCancel() override { return true; }
	virtual bool ExecuteNestedCancelCommand() override { return false; }

	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> ClickDragBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior = nullptr;

protected:
	FViewCameraState CameraState;
	
	bool bShiftToggle = false;
	bool bCtrlToggle = false;
};

// -------------------------------------------------------

UCLASS()
class ULidarEditorToolBuilderSelect : public ULidarEditorToolBuilderBase
{
	GENERATED_BODY()
};

UCLASS()
class ULidarToolActionsAlign : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UFUNCTION(CallInEditor, Category = Actions)
	void AlignAroundWorldOrigin();

	UFUNCTION(CallInEditor, Category = Actions)
	void AlignAroundOriginalCoordinates();

	UFUNCTION(CallInEditor, Category = Actions)
	void ResetAlignment();
};

UCLASS()
class ULidarEditorToolAlign : public ULidarEditorToolBase
{
public:
	GENERATED_BODY()
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override { return NewObject<ULidarToolActionsAlign>(this); }
};

UCLASS()
class ULidarEditorToolBuilderAlign : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolAlign>(SceneState.ToolManager); }
};

UCLASS()
class ULidarToolActionsMerge : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	bool bReplaceSourceActorsAfterMerging = false;
	
	UFUNCTION(CallInEditor, Category = Actions)
	void MergeActors();

	UFUNCTION(CallInEditor, Category = Actions)
	void MergeData();
};

UCLASS()
class ULidarEditorToolMerge : public ULidarEditorToolBase
{
public:
	GENERATED_BODY()
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override { return NewObject<ULidarToolActionsMerge>(this); }
};

UCLASS()
class ULidarEditorToolBuilderMerge : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolMerge>(SceneState.ToolManager); }
};

UCLASS()
class ULidarToolActionsCollision : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "2000", ClampMin = "0"))
	float OverrideMaxCollisionError = 0;
	
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "(Re-)Build Collision"))
	void BuildCollision();

	UFUNCTION(CallInEditor, Category = Actions)
	void RemoveCollision();
};

UCLASS()
class ULidarEditorToolCollision : public ULidarEditorToolBase
{
public:
	GENERATED_BODY()
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override { return NewObject<ULidarToolActionsCollision>(this); }
};

UCLASS()
class ULidarEditorToolBuilderCollision : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolCollision>(SceneState.ToolManager); }
};

UCLASS()
class ULidarToolActionsMeshing : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Max error around the meshed areas. Leave at 0 for max quality */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "2000", ClampMin = "0"))
	float MaxMeshingError = 0;
	
	UPROPERTY(EditAnywhere, Category = Options)
	bool bMergeMeshes = true;

	/** If not merging meshes, this will retain the transform of the original cloud */
	UPROPERTY(EditAnywhere, Category = Options, meta=(EditCondition="!bMergeMeshes"))
	bool bRetainTransform = true;
	
	UFUNCTION(CallInEditor, Category = Actions)
	void BuildStaticMesh();
};

UCLASS()
class ULidarEditorToolMeshing : public ULidarEditorToolBase
{
public:
	GENERATED_BODY()
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override { return NewObject<ULidarToolActionsMeshing>(this); }
};

UCLASS()
class ULidarEditorToolBuilderMeshing : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolMeshing>(SceneState.ToolManager); }
};

UCLASS()
class ULidarToolActionsNormals : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Higher values will generally result in more accurate calculations, at the expense of time */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (ClampMin = "1", ClampMax = "100"))
	int32 Quality = 40;

	/**
	 * Higher values are less susceptible to noise, but will most likely lose finer details, especially around hard edges.
	 * Lower values retain more detail, at the expense of time.
	 * NOTE: setting this too low will cause visual artifacts and geometry holes in noisier datasets.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options", meta = (ClampMin = "0.0"))
	float NoiseTolerance = 1;
	
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "(Re-)Build Normals"))
	void CalculateNormals();
};

UCLASS()
class ULidarEditorToolNormals : public ULidarEditorToolBase
{
public:
	GENERATED_BODY()
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override { return NewObject<ULidarToolActionsNormals>(this); }
};

UCLASS()
class ULidarEditorToolBuilderNormals : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolNormals>(SceneState.ToolManager); }
};

UCLASS()
class ULidarToolActionsSelection : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayName = "Clear"))
	void ClearSelection();
	
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayName = "Invert"))
	void InvertSelection();

	UFUNCTION(CallInEditor, Category = Cleanup)
	void DeleteSelected();
	
	UFUNCTION(CallInEditor, Category = Cleanup)
	void DeleteHidden();
	
	UFUNCTION(CallInEditor, Category = Cleanup)
	void HideSelected();

	UFUNCTION(CallInEditor, Category = Cleanup)
	void ResetVisibility();
	
	/** Higher values will generally result in more accurate calculations, at the expense of time */
	UPROPERTY(EditAnywhere, Category = "Normals", meta = (ClampMin = "1", ClampMax = "100"))
	int32 Quality = 40;

	/**
	 * Higher values are less susceptible to noise, but will most likely lose finer details, especially around hard edges.
	 * Lower values retain more detail, at the expense of time.
	 * NOTE: setting this too low will cause visual artifacts and geometry holes in noisier datasets.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Normals", meta = (ClampMin = "0.0"))
	float NoiseTolerance = 1;
	
	UFUNCTION(CallInEditor, Category = Normals)
	void CalculateNormals();
	
	UFUNCTION(CallInEditor, Category = "Merge & Extract")
	void Extract();

	UFUNCTION(CallInEditor, Category = "Merge & Extract")
	void ExtractAsCopy();
	
	/** Max error around the meshed areas. Leave at 0 for max quality */
	UPROPERTY(EditAnywhere, Category = Meshing, meta = (UIMin = "0", UIMax = "2000", ClampMin = "0"))
	float MaxMeshingError = 0;

	UPROPERTY(EditAnywhere, Category = Meshing)
	bool bMergeMeshes = true;

	/** If not merging meshes, this will retain the transform of the original cloud */
	UPROPERTY(EditAnywhere, Category = Meshing, meta=(EditCondition="!bMergeMeshes"))
	bool bRetainTransform = true;

	UFUNCTION(CallInEditor, Category = Meshing, meta = (DisplayName = "Create Static Mesh"))
	void BuildStaticMesh();
};

UCLASS()
class ULidarEditorToolSelectionBase : public ULidarEditorToolClickDragBase
{
public:
	GENERATED_BODY()
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	virtual TArray<FConvexVolume> GetSelectionConvexVolumes();
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;
	virtual bool ExecuteNestedCancelCommand() override;
	virtual FText GetToolMessage() const override;
	virtual FLinearColor GetHUDColor();
	virtual void FinalizeSelection();

	virtual void PostCurrentMousePosChanged() {}
	virtual ELidarPointCloudSelectionMode GetSelectionMode() const;

protected:
	FVector2d CurrentMousePos;
	TArray<FVector2d> Clicks;
	bool bSelecting;
};

UCLASS()
class ULidarEditorToolBoxSelection : public ULidarEditorToolSelectionBase
{
public:
	GENERATED_BODY()
	virtual TArray<FConvexVolume> GetSelectionConvexVolumes() override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
};

UCLASS()
class ULidarEditorToolBuilderBoxSelection : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolBoxSelection>(SceneState.ToolManager); }
};

UCLASS()
class ULidarEditorToolPolygonalSelection : public ULidarEditorToolSelectionBase
{
public:
	GENERATED_BODY()
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual FLinearColor GetHUDColor() override;
	virtual void PostCurrentMousePosChanged() override;

private:
	bool IsWithinSnap();
};

UCLASS()
class ULidarEditorToolBuilderPolygonalSelection : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolPolygonalSelection>(SceneState.ToolManager); }
};

UCLASS()
class ULidarEditorToolLassoSelection : public ULidarEditorToolSelectionBase
{
public:
	GENERATED_BODY()
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
};

UCLASS()
class ULidarEditorToolBuilderLassoSelection : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolLassoSelection>(SceneState.ToolManager); }
};

UCLASS()
class ULidarToolActionsPaintSelection : public ULidarToolActionsSelection
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = "0", UIMax = "8196", ClampMin = "0", DisplayPriority = 1))
	float BrushRadius = 250;
};

UCLASS()
class ULidarEditorToolPaintSelection : public ULidarEditorToolSelectionBase
{
public:
	GENERATED_BODY()
	virtual void Setup() override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override {}
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void PostCurrentMousePosChanged() override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual TObjectPtr<UInteractiveToolPropertySet> CreateToolActions() override { return NewObject<ULidarToolActionsPaintSelection>(this); }
	virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

private:
	void Paint();

public:
	float BrushRadius;
	
private:
	FVector3f HitLocation;
	float LastHitDistance;
	bool bHasHit;
};

UCLASS()
class ULidarEditorToolBuilderPaintSelection : public ULidarEditorToolBuilderBase
{
public:
	GENERATED_BODY()
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override { return NewObject<ULidarEditorToolPaintSelection>(SceneState.ToolManager); }
};