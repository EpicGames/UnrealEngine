// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "LidarPointCloudEditorViewport.h"
#include "LidarPointCloudEditorViewportClient.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudComponent.h"
#include "LidarPointCloudEditorCommands.h"

#include "Slate/SceneViewport.h"
#include "Styling/AppStyle.h"
#include "ComponentReregisterContext.h"
#include "Widgets/Docking/SDockTab.h"

///////////////////////////////////////////////////////////
// SPointCloudEditorViewportToolbar

// In-viewport toolbar widget used in the static mesh editor
class SPointCloudEditorViewportToolbar : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(SPointCloudEditorViewportToolbar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
	{
		SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
	}

	// SCommonEditorViewportToolbarBase interface
	virtual TSharedRef<SWidget> GenerateShowMenu() const override
	{
		GetInfoProvider().OnFloatingButtonClicked();

		TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
		{
			const FLidarPointCloudEditorCommands& Commands = FLidarPointCloudEditorCommands::Get();

			ShowMenuBuilder.AddMenuEntry(Commands.SetShowGrid);
			ShowMenuBuilder.AddMenuEntry(Commands.SetShowBounds);
			ShowMenuBuilder.AddMenuEntry(Commands.SetShowCollision);
			ShowMenuBuilder.AddMenuEntry(Commands.SetShowNodes);
		}

		return ShowMenuBuilder.MakeWidget();
	}
	// End of SCommonEditorViewportToolbarBase
};

///////////////////////////////////////////////////////////
// SLidarPointCloudEditorViewport

void SLidarPointCloudEditorViewport::Construct(const FArguments& InArgs)
{
	PointCloudEditorPtr = InArgs._PointCloudEditor;

	PointCloud = InArgs._ObjectToEdit;

	CurrentViewMode = VMI_Lit;

	PreviewCloudComponent = NewObject<ULidarPointCloudComponent>();

	SEditorViewport::Construct(SEditorViewport::FArguments());

	FComponentReregisterContext ReregisterContext(PreviewCloudComponent);

	PreviewScene.AddComponent(PreviewCloudComponent, FTransform::Identity);

	SetPreviewCloud(PointCloud);
	
	ViewportOverlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(FMargin(4.0f, 35.0f))
		[
			SAssignNew(OverlayTextBackground, SBorder)
				.BorderBackgroundColor(FSlateColor(FLinearColor(0,0,0,0)))
				.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				.Padding(FMargin(10.0f))
				[
					SAssignNew(OverlayTextVerticalBox, SVerticalBox)
				]
		];
}

SLidarPointCloudEditorViewport::SLidarPointCloudEditorViewport()
	: PreviewScene(FPreviewScene::ConstructionValues())
{
}

SLidarPointCloudEditorViewport::~SLidarPointCloudEditorViewport()
{
	if (PreviewCloudComponent)
	{
		PreviewScene.RemoveComponent(PreviewCloudComponent);
		PreviewCloudComponent->SetPointCloud(nullptr);
		PreviewCloudComponent = nullptr;
	}
}

void SLidarPointCloudEditorViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PreviewCloudComponent);
	Collector.AddReferencedObject(PointCloud);
}

void SLidarPointCloudEditorViewport::RefreshViewport()
{
	// Invalidate the viewport's display.
	SceneViewport->Invalidate();
}

void SLidarPointCloudEditorViewport::ResetCamera()
{
	if (FLidarPointCloudEditorViewportClient* EditorViewportClientRawPtr = EditorViewportClient.Get())
	{
		EditorViewportClientRawPtr->ResetCamera();
	}
}

void SLidarPointCloudEditorViewport::SetPreviewCloud(ULidarPointCloud* InPointCloud)
{
	// Set the new preview point cloud.
	PreviewCloudComponent->SetPointCloud(InPointCloud);
	PreviewCloudComponent->OwningViewportClient = TWeakPtr<FViewportClient>(EditorViewportClient);
	PreviewCloudComponent->MarkRenderStateDirty();

	ResetCamera();
}

void SLidarPointCloudEditorViewport::PopulateOverlayText(const TArray<FOverlayTextItem>& TextItems)
{
	OverlayTextVerticalBox->ClearChildren();

	for (const FOverlayTextItem& TextItem : TextItems)
	{
		OverlayTextVerticalBox->AddSlot()
			[
				SNew(STextBlock)
				.Text(TextItem.Text)
				.TextStyle(FAppStyle::Get(), TextItem.Style)
			];
	}

	OverlayTextBackground->SetBorderBackgroundColor(FSlateColor(FLinearColor(0, 0, 0, TextItems.Num() > 0 ? 0.45f : 0)));
}

bool SLidarPointCloudEditorViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground());
}

TSharedRef<FEditorViewportClient> SLidarPointCloudEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FLidarPointCloudEditorViewportClient(PointCloudEditorPtr, SharedThis(this), &PreviewScene, PreviewCloudComponent));

	EditorViewportClient->bSetListenerPosition = false;

	EditorViewportClient->SetRealtime(true);
	EditorViewportClient->VisibilityDelegate.BindSP(this, &SLidarPointCloudEditorViewport::IsVisible);

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SLidarPointCloudEditorViewport::BuildViewportToolbar()
{
	return SNew(SPointCloudEditorViewportToolbar, SharedThis(this));
}

void SLidarPointCloudEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FLidarPointCloudEditorCommands& Commands = FLidarPointCloudEditorCommands::Get();

	TSharedRef<FLidarPointCloudEditorViewportClient> EditorViewportClientRef = EditorViewportClient.ToSharedRef();

	CommandList->MapAction(
		Commands.ResetCamera,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::ResetCamera));

	CommandList->MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::SetShowGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::IsSetShowGridChecked));

	CommandList->MapAction(
		Commands.SetShowBounds,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::ToggleShowBounds),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::IsSetShowBoundsChecked));

	CommandList->MapAction(
		Commands.SetShowCollision,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::SetShowCollision),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::IsSetShowCollisionChecked));

	CommandList->MapAction(
		Commands.SetShowNodes,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::ToggleShowNodes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FLidarPointCloudEditorViewportClient::IsSetShowNodesChecked));
}

void SLidarPointCloudEditorViewport::OnFocusViewportToSelection()
{
	if (PreviewCloudComponent)
	{
		EditorViewportClient->FocusViewportOnBox(PreviewCloudComponent->Bounds.GetBox());
	}
}
