// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "CoreMinimal.h"
#include "EditorViewportClient.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"
#include "UObject/GCObject.h"
#include "Widgets/Layout/SBorder.h"

class FLidarPointCloudEditor;
class FLidarPointCloudEditorViewportClient;
class SVerticalBox;
class ULidarPointCloud;
class ULidarPointCloudComponent;

/**
 * PointCloud Editor Preview viewport widget
 */
class SLidarPointCloudEditorViewport : public SEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SLidarPointCloudEditorViewport) {}
		SLATE_ARGUMENT(TWeakPtr<FLidarPointCloudEditor>, PointCloudEditor)
		SLATE_ARGUMENT(ULidarPointCloud*, ObjectToEdit)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	SLidarPointCloudEditorViewport();
	~SLidarPointCloudEditorViewport();

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SLidarPointCloudEditorViewport");
	}
	// End of FGCObject interface

	void RefreshViewport();

	void ResetCamera();

	/** Component for the preview point cloud. */
	TObjectPtr<ULidarPointCloudComponent> PreviewCloudComponent;
	
	/**
	 *	Sets up the point cloud that the Point Cloud editor is viewing.
	 *
	 *	@param	InPointCloud		The point cloud being viewed in the editor.
	 */
	void SetPreviewCloud(ULidarPointCloud* InPointCloud);

	/** Set the parent tab of the viewport for determining visibility */
	void SetParentTab(TSharedRef<SDockTab> InParentTab) { ParentTab = InParentTab; }

	/** Struct defining the text and its style of each item in the overlay widget */
	struct FOverlayTextItem
	{
		explicit FOverlayTextItem(const FText& InText, const FName& InStyle = "TextBlock.ShadowedText")
			: Text(InText), Style(InStyle)
		{}

		FText Text;
		FName Style;
	};

	/** Specifies an array of text items which will be added to the viewport overlay */
	void PopulateOverlayText(const TArray<FOverlayTextItem>& TextItems);

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override { return SharedThis(this); }
	virtual TSharedPtr<FExtender> GetExtenders() const override { return TSharedPtr<FExtender>(MakeShareable(new FExtender)); }
	virtual void OnFloatingButtonClicked() override {}
	// End of ICommonEditorViewportToolbarInfoProvider interface

	TSharedPtr<FLidarPointCloudEditorViewportClient> GetEditorViewportClient() { return EditorViewportClient; }

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual EVisibility OnGetViewportContentVisibility() const override { return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed; }
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;

private:
	/** Determines the visibility of the viewport. */
	virtual bool IsVisible() const override;

private:
	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

	/** Pointer back to the PointCloud editor tool that owns us */
	TWeakPtr<FLidarPointCloudEditor> PointCloudEditorPtr;

	/** The scene for this viewport. */
	FPreviewScene PreviewScene;

	/** Editor viewport client */
	TSharedPtr<FLidarPointCloudEditorViewportClient> EditorViewportClient;

	/** Point Cloud being edited */
	TObjectPtr<ULidarPointCloud> PointCloud;

	/** The currently selected view mode. */
	EViewModeIndex CurrentViewMode;

	/** Pointer to the vertical box into which the overlay text items are added */
	TSharedPtr<SVerticalBox> OverlayTextVerticalBox;

	/** Pointer to the background of the overlay text */
	TSharedPtr<SBorder> OverlayTextBackground;
};
