// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudEditorViewportClient.h"
#include "LidarPointCloudEditor.h"
#include "LidarPointCloudEditorViewport.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudComponent.h"

#include "ConvexVolume.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "AssetViewerSettings.h"

#include "GeomTools.h"

#define LOCTEXT_NAMESPACE "FLidarPointCloudEditorViewportClient"

namespace UE {
	namespace Lidar {
		namespace Private {
			namespace Editor
			{
				constexpr float GridSize = 2048.0f;
				constexpr int32 CellSize = 16;
			}
		}
	}
}

FLidarPointCloudEditorViewportClient::FLidarPointCloudEditorViewportClient(TWeakPtr<FLidarPointCloudEditor> InPointCloudEditor, const TSharedRef<SLidarPointCloudEditorViewport>& InPointCloudEditorViewport, FPreviewScene* InPreviewScene, ULidarPointCloudComponent* InPreviewPointCloudComponent)
	: FEditorViewportClient(nullptr, InPreviewScene, StaticCastSharedRef<SEditorViewport>(InPointCloudEditorViewport))
	, PointCloudComponent(InPreviewPointCloudComponent)
	, PointCloudEditorPtr(InPointCloudEditor)
	, PointCloudEditorViewportPtr(InPointCloudEditorViewport)
{
	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(160, 160, 160);
	DrawHelper.GridColorMajor = FColor(144, 144, 144);
	DrawHelper.GridColorMinor = FColor(128, 128, 128);
	DrawHelper.PerspectiveGridSize = UE::Lidar::Private::Editor::GridSize;
	DrawHelper.NumCells = DrawHelper.PerspectiveGridSize / (UE::Lidar::Private::Editor::CellSize * 2);

	SetViewMode(VMI_Unlit);

	EngineShowFlags.SetSeparateTranslucency(true);
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = true;

	// Set correct flags according to current profile settings
	SetAdvancedShowFlagsForScene(UAssetViewerSettings::Get()->Profiles[GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex].bPostProcessingEnabled);
}

FLidarPointCloudEditorViewportClient::~FLidarPointCloudEditorViewportClient()
{
	UAssetViewerSettings::Get()->OnAssetViewerSettingsChanged().RemoveAll(this);
}

void FLidarPointCloudEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	TArray<SLidarPointCloudEditorViewport::FOverlayTextItem> TextItems;

	if (TSharedPtr<FLidarPointCloudEditor> Editor = PointCloudEditorPtr.Pin())
	{
		if(const ULidarPointCloud* PC = Editor->GetPointCloudBeingEdited())
		{
			const FIntVector BoundingSize(PC->GetBounds().GetSize());

#define AS_NUM(Val) FText::AsNumber(Val, &FNumberFormattingOptions::DefaultWithGrouping())
			TextItems.Emplace(FText::Format(LOCTEXT("PointCount", "Point Count: {0}"), AS_NUM(PC->GetNumPoints())));
			TextItems.Emplace(FText::Format(LOCTEXT("NodeCount", "Node Count: {0}"), AS_NUM(PC->GetNumNodes())));
			TextItems.Emplace(FText::Format(LOCTEXT("Bounds", "Bounds: {0} x {1} x {2}"), AS_NUM(BoundingSize.X), AS_NUM(BoundingSize.Y), AS_NUM(BoundingSize.Z)));
			TextItems.Emplace(FText::Format(LOCTEXT("Collider", "Collider: {0} poly"), AS_NUM(PC->GetColliderPolys())));
			TextItems.Emplace(FText::Format(LOCTEXT("DataSize", "Data Size: {0} MB"), AS_NUM(PC->GetDataSize())));
#undef AS_NUM
		}
	}

	if (TSharedPtr<SLidarPointCloudEditorViewport> PointCloudEditorViewport = PointCloudEditorViewportPtr.Pin())
	{
		PointCloudEditorViewport->PopulateOverlayText(TextItems);
	}
}

bool FLidarPointCloudEditorViewportClient::ShouldOrbitCamera() const
{
	return GetDefault<ULevelEditorViewportSettings>()->bUseUE3OrbitControls || FEditorViewportClient::ShouldOrbitCamera();
}

void FLidarPointCloudEditorViewportClient::PerspectiveCameraMoved()
{
	FEditorViewportClient::PerspectiveCameraMoved();

	// If in the process of transitioning to a new location, don't update the orbit camera position.
	// On the final update of the transition, we will get here with IsPlaying()==false, and the editor camera position will
	// be correctly updated.
	if (GetViewTransform().IsPlaying())
	{
		return;
	}

	ToggleOrbitCamera(bUsingOrbitCamera);
}

void FLidarPointCloudEditorViewportClient::SetAdvancedShowFlagsForScene(const bool bAdvancedShowFlags)
{
	if (bAdvancedShowFlags)
	{
		EngineShowFlags.EnableAdvancedFeatures();
	}
	else
	{
		EngineShowFlags.DisableAdvancedFeatures();
	}
}

void FLidarPointCloudEditorViewportClient::ResetCamera()
{
	if (ULidarPointCloudComponent* PointCloudComponentRawPtr = PointCloudComponent.Get())
	{
		FocusViewportOnBox(PointCloudComponentRawPtr->Bounds.GetBox());
		Invalidate();
	}
}

void FLidarPointCloudEditorViewportClient::ToggleShowNodes()
{
	if (ULidarPointCloudComponent* PointCloudComponentRawPtr = PointCloudComponent.Get())
	{
		PointCloudComponentRawPtr->bDrawNodeBounds = !PointCloudComponentRawPtr->bDrawNodeBounds;
		Invalidate();
	}
}

bool FLidarPointCloudEditorViewportClient::IsSetShowNodesChecked() const
{
	if (ULidarPointCloudComponent* PointCloudComponentRawPtr = PointCloudComponent.Get())
	{
		return PointCloudComponentRawPtr->bDrawNodeBounds;
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE 
