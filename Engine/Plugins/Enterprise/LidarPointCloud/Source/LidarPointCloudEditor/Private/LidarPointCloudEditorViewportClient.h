// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UnrealWidgetFwd.h"
#include "EditorViewportClient.h"
#include "LidarPointCloudEditorHelper.h"
#include "LidarPointCloudShared.h"

class FAdvancedPreviewScene;
class FCanvas;
class FLidarPointCloudEditor;
class SLidarPointCloudEditorViewport;
class ULidarPointCloud;
class ULidarPointCloudComponent;

/** Viewport Client for the preview viewport */
class FLidarPointCloudEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FLidarPointCloudEditorViewportClient>
{
public:
	FLidarPointCloudEditorViewportClient(TWeakPtr<FLidarPointCloudEditor> InPointCloudEditor, const TSharedRef<SLidarPointCloudEditorViewport>& InPointCloudEditorViewport, FPreviewScene* InPreviewScene, ULidarPointCloudComponent* InPreviewPointCloudComponent);
	~FLidarPointCloudEditorViewportClient();

	// FEditorViewportClient interface
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual bool InputWidgetDelta(FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale) override { return false; }
	virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override {}
	virtual void TrackingStopped() override {}
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override { return UE::Widget::WM_None; }
	virtual void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override {}
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }
	virtual FVector GetWidgetLocation() const override { return FVector::ZeroVector; }
	virtual FMatrix GetWidgetCoordSystem() const override { return FMatrix::Identity; }
	virtual ECoordSystem GetWidgetCoordSystemSpace() const override { return COORD_Local; }
	virtual bool ShouldOrbitCamera() const override;
	
	void ResetCamera();

	/** Callback for toggling the nodes show flag. */
	void ToggleShowNodes();

	/** Callback for checking the nodes show flag. */
	bool IsSetShowNodesChecked() const;

protected:
	// FEditorViewportClient interface
	virtual void PerspectiveCameraMoved() override;

	/** Used to (re)-set the viewport show flags related to post processing*/
	void SetAdvancedShowFlagsForScene(const bool bAdvancedShowFlags);

private:
	/** Component for the point cloud. */
	TWeakObjectPtr<ULidarPointCloudComponent> PointCloudComponent;

	/** Pointer back to the PointCloud editor tool that owns us */
	TWeakPtr<FLidarPointCloudEditor> PointCloudEditorPtr;

	/** Pointer back to the PointCloudEditor viewport control that owns us */
	TWeakPtr<SLidarPointCloudEditorViewport> PointCloudEditorViewportPtr;
};
