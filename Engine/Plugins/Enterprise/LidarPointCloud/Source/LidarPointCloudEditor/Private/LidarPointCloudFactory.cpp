// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudFactory.h"

#include "ContentBrowserMenuContexts.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudSettings.h"
#include "LidarPointCloudImportUI.h"
#include "IO/LidarPointCloudFileIO.h"
#include "Editor.h"

#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "LidarPointCloud"




ULidarPointCloudFactory::ULidarPointCloudFactory()
{
	bImportingAll = false;
	bCreateNew = true;
	bEditorImport = true;
	SupportedClass = ULidarPointCloud::StaticClass();

	TArray<FString> exts = ULidarPointCloudFileIO::GetSupportedImportExtensions();
	for (FString ext : exts)
	{
		Formats.Add(*ext.Append(";LiDAR Point Cloud"));
	}
}

UObject* ULidarPointCloudFactory::ImportObject(UClass* InClass, UObject* InOuter, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, bool& OutCanceled)
{
	bCreateNew = false;
	UObject* NewPC = Super::ImportObject(InClass, InOuter, InName, Flags, Filename, Parms, OutCanceled);
	bCreateNew = true;

	return NewPC;
}

UObject* ULidarPointCloudFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UObject *OutObject = nullptr;

	UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>();
	ImportSubsystem->OnAssetPreImport.Broadcast(this, InClass, InParent, InName, *FPaths::GetExtension(Filename));

	// Check if the headers differ between files. Log occurrence and prompt the user for interaction if necessary
	if (bImportingAll && ImportSettings.IsValid())
	{
		if (!ImportSettings->IsFileCompatible(Filename))
		{
			PC_WARNING("Inconsistent header information between files - batch import cancelled.");
			bImportingAll = false;
		}
		else
		{
			ImportSettings->SetNewFilename(Filename);
		}
	}

	if (!bImportingAll)
	{
		ImportSettings = FLidarPointCloudImportUI::ShowImportDialog(Filename, false);
		bImportingAll = ImportSettings.IsValid() && ImportSettings->bImportAll;
	}

	if (ImportSettings.IsValid())
	{
		OutObject = ULidarPointCloud::CreateFromFile(Filename, ImportSettings->Clone(), InParent, InName, Flags);
	}
	else
	{
		bOutOperationCanceled = true;
	}

	ImportSubsystem->OnAssetPostImport.Broadcast(this, OutObject);

	return OutObject;
}

UObject* ULidarPointCloudFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<ULidarPointCloud>(InParent, InName, Flags);
}

bool ULidarPointCloudFactory::DoesSupportClass(UClass* Class) { return Class == ULidarPointCloud::StaticClass(); }

bool ULidarPointCloudFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{	
	ULidarPointCloud* PC = Cast<ULidarPointCloud>(Obj);
	if (PC)
	{
		OutFilenames.Add(*PC->GetSourcePath());
		return true;
	}
	return false;
}

void ULidarPointCloudFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	ULidarPointCloud *PC = Cast<ULidarPointCloud>(Obj);
	if (PC && NewReimportPaths.Num())
	{
		PC->SetSourcePath(NewReimportPaths[0]);
	}
}

EReimportResult::Type ULidarPointCloudFactory::Reimport(UObject* Obj)
{
	ULidarPointCloud *PC = Cast<ULidarPointCloud>(Obj);

	if (PC)
	{	
		bool bSuccess = false;

		// Show existing settings, if the cloud has any
		if (PC->ImportSettings.IsValid())
		{
			bSuccess = FLidarPointCloudImportUI::ShowImportDialog(PC->ImportSettings, true);
		}
		// ... otherwise attempt to generate new, based on the source path (if valid)
		else if (FPaths::FileExists(PC->GetSourcePath()))
		{
			PC->ImportSettings = FLidarPointCloudImportUI::ShowImportDialog(PC->GetSourcePath(), true);
			bSuccess = PC->ImportSettings.IsValid();
		}
		else
		{
			PC_ERROR("Cannot reimport, source path is incorrect.");
		}

		if (bSuccess)
		{
			PC->Reimport(GetDefault<ULidarPointCloudSettings>()->bUseAsyncImport);
		}
	}

	// Return cancelled, to avoid showing 2 notifications
	return EReimportResult::Cancelled;
}

#undef LOCTEXT_NAMESPACE