// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "EditorReimportHandler.h"
#include "AssetTypeActions_Base.h"
#include "Factories/Factory.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudFactory.generated.h"


UCLASS()
class ULidarPointCloudFactory : public UFactory, public FReimportHandler
{
	GENERATED_BODY()

private:
	TSharedPtr<struct FLidarPointCloudImportSettings> ImportSettings;
	bool bImportingAll;

public:
	ULidarPointCloudFactory();

	// Begin UFactory Interface
	virtual UObject* ImportObject(UClass* InClass, UObject* InOuter, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, bool& OutCanceled) override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool DoesSupportClass(UClass* Class) override;
	virtual bool FactoryCanImport(const FString& Filename) override { return true; }
	virtual FText GetDisplayName() const override { return FText::FromString("LiDAR Point Cloud"); }
	// End UFactory Interface

	// Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	// End FReimportHandler Interface
};