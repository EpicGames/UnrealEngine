// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudImportUI.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "IO/LidarPointCloudFileIO.h"

#define LOCTEXT_NAMESPACE "LidarPointCloudImportUI"

/////////////////////////////////////////////////
// SLidarPointCloudOptionWindow

void SLidarPointCloudOptionWindow::Construct(const FArguments& InArgs)
{
	bool bIsReimport = InArgs._IsReimport;

	ImportUI = InArgs._ImportUI;
	WidgetWindow = InArgs._WidgetWindow;

	check (ImportUI.IsValid());
	
	ImportAllButton = SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("LidarPointCloudOptionWindow_ImportAll", "Import All"))
					.ToolTipText(LOCTEXT("LidarPointCloudOptionWindow_ImportAll_ToolTip", "Import all files with these same settings"))
					.IsEnabled(this, &SLidarPointCloudOptionWindow::CanImport)
					.OnClicked(this, &SLidarPointCloudOptionWindow::OnImportAll);

	ImportButton = SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(bIsReimport ? LOCTEXT("LidarPointCloudOptionWindow_Reimport", "Reimport") : LOCTEXT("LidarPointCloudOptionWindow_Import", "Import"))
					.IsEnabled(this, &SLidarPointCloudOptionWindow::CanImport)
					.OnClicked(this, &SLidarPointCloudOptionWindow::OnImport);

	CancelButton = SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("LidarPointCloudOptionWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("LidarPointCloudOptionWindow_Cancel_ToolTip", "Cancels importing this Point Cloud file"))
					.OnClicked(this, &SLidarPointCloudOptionWindow::OnCancel);						

	TSharedPtr<SHorizontalBox> HeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
						.Text(LOCTEXT("Import_CurrentFileTitle", "Current File: "))
					]
					+SHorizontalBox::Slot()
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
						.Text(InArgs._FullPath)
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SAssignNew(InspectorBox, SBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2)
			[
				bIsReimport ?
				(
					SNew(SUniformGridPanel)
					.SlotPadding(2)
					+ SUniformGridPanel::Slot(0, 0)
					[
						ImportButton->AsShared()
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						CancelButton->AsShared()
					]
				)
				:
				(
					SNew(SUniformGridPanel)
					.SlotPadding(2)
					+ SUniformGridPanel::Slot(0, 0)
					[
						ImportAllButton->AsShared()
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						ImportButton->AsShared()
					]
					+ SUniformGridPanel::Slot(2, 0)
					[
						CancelButton->AsShared()
					]
				)
			]
		]
	];

	// Apply customized widget
	TSharedPtr<SWidget> SettingsWidget = ImportUI.Get()->GetWidget();
	if (SettingsWidget.IsValid())
	{
		InspectorBox->SetContent(SettingsWidget.ToSharedRef());
	}
}

/////////////////////////////////////////////////
// FLidarPointCloudImportUI

TSharedPtr<FLidarPointCloudImportSettings> FLidarPointCloudImportUI::ShowImportDialog(const FString& Filename, bool bIsReimport)
{
	TSharedPtr<FLidarPointCloudImportSettings> ImportUI = ULidarPointCloudFileIO::GetImportSettings(Filename);

	if (!ShowImportDialog(ImportUI, bIsReimport))
	{
		ImportUI = nullptr;
	}

	return ImportUI;
}

bool FLidarPointCloudImportUI::ShowImportDialog(TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, bool bIsReimport)
{
	if (!ImportSettings.IsValid())
	{
		return false;
	}

	if (!ImportSettings->HasImportUI())
	{
		return true;
	}

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UnrealEd", "LidarPointCloudImportOpionsTitle", "LiDAR Point Cloud Import Options"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea);
	
	TSharedPtr<SLidarPointCloudOptionWindow> PointCloudOptionWindow;
	Window->SetContent
	(
		SAssignNew(PointCloudOptionWindow, SLidarPointCloudOptionWindow)
		.ImportUI(ImportSettings)
		.WidgetWindow(Window)
		.FullPath(FText::FromString(ImportSettings->GetFilename().Len() > 58 ? ("..." + ImportSettings->GetFilename().Right(55)) : ImportSettings->GetFilename()))
		.IsReimport(bIsReimport)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	ImportSettings->bImportAll = PointCloudOptionWindow->ShouldImportAll();

	return !PointCloudOptionWindow->bCancelled;
}

#undef LOCTEXT_NAMESPACE