// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"

struct FLidarPointCloudImportSettings;

class SLidarPointCloudOptionWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLidarPointCloudOptionWindow)
		: _ImportUI(nullptr)
		, _WidgetWindow()
		, _FullPath()
		, _IsReimport()
		, _MaxWindowHeight(0.0f)
		, _MaxWindowWidth(0.0f)
	{}

	SLATE_ARGUMENT(TSharedPtr<FLidarPointCloudImportSettings>, ImportUI)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(FText, FullPath)
	SLATE_ARGUMENT(bool, IsReimport)
	SLATE_ARGUMENT(float, MaxWindowHeight)
	SLATE_ARGUMENT(float, MaxWindowWidth)
	SLATE_END_ARGS()

public:
	bool bCancelled;

private:
	bool bImportAll;
	TSharedPtr<FLidarPointCloudImportSettings> ImportUI;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ImportButton;
	TSharedPtr<SButton> ImportAllButton;
	TSharedPtr<SButton> CancelButton;

public:
	SLidarPointCloudOptionWindow() : bCancelled(true), bImportAll(false), ImportUI(nullptr)  {}

	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	FReply OnImport()
	{
		bCancelled = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnImportAll()
	{
		bImportAll = true;
		return OnImport();
	}

	FReply OnCancel()
	{
		bImportAll = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	bool ShouldImportAll() { return bImportAll; }

private:
	bool CanImport() const { return true; };
};

class FLidarPointCloudImportUI
{
public:
	/**
	 * This version queries the default ImportSettings based on the Filename provided
	 * Returns the ImportSettings object
	 */
	static TSharedPtr<FLidarPointCloudImportSettings> ShowImportDialog(const FString& Filename, bool bIsReimport);

	/**
	 * This version uses existing ImportSettings
	 * Returns true if ImportSettings are valid and the action has been accepted by the user
	 */
	static bool ShowImportDialog(TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, bool bIsReimport);
};