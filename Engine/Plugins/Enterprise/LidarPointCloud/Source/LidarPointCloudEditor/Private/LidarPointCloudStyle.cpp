// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( StyleSet->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( StyleSet->RootToContentDir( RelativePath, TEXT(".svg") ), __VA_ARGS__ )

TSharedPtr<FSlateStyleSet> FLidarPointCloudStyle::StyleSet = nullptr;
TSharedPtr<class ISlateStyle> FLidarPointCloudStyle::Get() { return StyleSet; }

void FLidarPointCloudStyle::Initialize()
{
	// Const icon & thumbnail sizes
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon128x128(128.0f, 128.0f);
	
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}
	
	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	
	StyleSet->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("LidarPointCloud"))->GetContentDir() / TEXT("Icons"));
	
	StyleSet->Set("ClassIcon.LidarPointCloud", new IMAGE_PLUGIN_BRUSH("icon_32", Icon16x16));
	StyleSet->Set("ClassIcon32.LidarPointCloud", new IMAGE_PLUGIN_BRUSH("icon_32", Icon32x32));
	StyleSet->Set("ClassThumbnail.LidarPointCloud", new IMAGE_PLUGIN_BRUSH("icon_128", Icon128x128));

	StyleSet->Set("ClassIcon.LidarPointCloudActor", new IMAGE_PLUGIN_BRUSH("icon_32", Icon16x16));
	StyleSet->Set("ClassThumbnail.LidarPointCloudActor", new IMAGE_PLUGIN_BRUSH("icon_128", Icon128x128));

	StyleSet->Set("ClassIcon.LidarPointCloudComponent", new IMAGE_PLUGIN_BRUSH("icon_32", Icon16x16));
	StyleSet->Set("ClassThumbnail.LidarPointCloudComponent", new IMAGE_PLUGIN_BRUSH("icon_128", Icon128x128));

	StyleSet->Set("ClassIcon.LidarClippingVolume", new IMAGE_PLUGIN_BRUSH("icon_32", Icon16x16));
	StyleSet->Set("ClassIcon32.LidarClippingVolume", new IMAGE_PLUGIN_BRUSH("icon_32", Icon32x32));
	StyleSet->Set("ClassThumbnail.LidarClippingVolume", new IMAGE_PLUGIN_BRUSH("icon_128", Icon128x128));
	
	StyleSet->Set("LidarPointCloudEditor.ToolkitCollision", new IMAGE_PLUGIN_BRUSH_SVG("Collision_20", Icon20x20));
	StyleSet->Set("LidarPointCloudEditor.ToolkitMerge", new IMAGE_PLUGIN_BRUSH_SVG("Merge_20", Icon20x20));
	StyleSet->Set("LidarPointCloudEditor.ToolkitAlign", new IMAGE_PLUGIN_BRUSH_SVG("AlignLidar", Icon20x20));
	StyleSet->Set("LidarPointCloudEditor.ToolkitBoxSelection", new IMAGE_PLUGIN_BRUSH_SVG("BoxSelection", Icon20x20));
	StyleSet->Set("LidarPointCloudEditor.ToolkitPolygonalSelection", new IMAGE_PLUGIN_BRUSH_SVG("PolySelection", Icon20x20));
	StyleSet->Set("LidarPointCloudEditor.ToolkitLassoSelection", new IMAGE_PLUGIN_BRUSH_SVG("Lasso", Icon20x20));
	StyleSet->Set("LidarPointCloudEditor.ToolkitNormals", new IMAGE_PLUGIN_BRUSH_SVG( TEXT("SetShowNormals_20"), Icon20x20 ));
	
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	
	StyleSet->Set("LidarPointCloudEditor.ToolkitSelect", new IMAGE_PLUGIN_BRUSH( TEXT("Icons/GeneralTools/Select_40x"), Icon20x20 ));
	StyleSet->Set("LidarPointCloudEditor.ToolkitPaintSelection", new IMAGE_PLUGIN_BRUSH_SVG( TEXT("Starship/Common/Paintbrush"), Icon20x20));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

#undef IMAGE_PLUGIN_BRUSH

void FLidarPointCloudStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FLidarPointCloudStyle::GetStyleSetName()
{
	static FName PaperStyleName(TEXT("LidarPointCloudStyle"));
	return PaperStyleName;
}

