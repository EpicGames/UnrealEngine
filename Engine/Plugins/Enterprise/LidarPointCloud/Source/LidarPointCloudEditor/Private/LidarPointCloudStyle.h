// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FLidarPointCloudStyle
{
public:
	static void Initialize();

	static void Shutdown();

	static TSharedPtr< class ISlateStyle > Get();

	static FName GetStyleSetName();

private:
	static TSharedPtr<class FSlateStyleSet> StyleSet;
};
