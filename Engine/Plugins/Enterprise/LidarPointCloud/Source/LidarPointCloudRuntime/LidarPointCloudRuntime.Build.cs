// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class LidarPointCloudRuntime : ModuleRules
	{
		public LidarPointCloudRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"Core",
					"Engine",
					"RenderCore",
					"Projects",
					"RHI",
					"InputCore",
					"GeometryCore",
					"MeshConversion",
					"MeshDescription",
					"StaticMeshDescription"
				}
            );

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"EditorFramework",
						"UnrealEd",
						"PropertyEditor",
						"ContentBrowser",
						"AssetRegistry"
					}
				);
			}

			// Currently, E57 is only supported on Windows
			bool bSupportLibE57 = true;
			
			// Currently, LAZ is only supported on Windows and Mac
			bool bSupportLasZip = true;

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				if (bSupportLasZip)
				{
					RuntimeDependencies.Add(Path.Combine(PluginDirectory, "Source", "ThirdParty", "LasZip", "Win64", "laszip.dll"));
				}

				if(bSupportLibE57)
				{
					RuntimeDependencies.Add(Path.Combine(PluginDirectory, "Source", "ThirdParty", "LibE57", "Win64", "xerces-c_3_2.dll"));
					RuntimeDependencies.Add(Path.Combine(PluginDirectory, "Source", "ThirdParty", "LibE57", "Win64", "E57UE4.dll"));
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				if (bSupportLasZip)
				{
					RuntimeDependencies.Add(Path.Combine(PluginDirectory, "Source", "ThirdParty", "LasZip", "Mac", "laszip.dylib"));
				}
				
				bSupportLibE57 = false;
			}
			else
			{
				bSupportLasZip = false;
				bSupportLibE57 = false;
			}

			AppendStringToPublicDefinition("LASZIPSUPPORTED", bSupportLasZip ? "1" : "0");
			AppendStringToPublicDefinition("LIBE57SUPPORTED", bSupportLibE57 ? "1" : "0");
		}
	}
}
