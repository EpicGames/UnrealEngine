// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/LidarPointCloudFileIO.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloud.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

TMap<FString, FLidarPointCloudFileIOHandler*> ULidarPointCloudFileIO::RegisteredHandlers;

//////////////////////////////////////////////////////////// Settings

void FLidarPointCloudImportSettings::Serialize(FArchive& Ar)
{
	int32 Dummy;
	bool bDummy = false;

	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 12)
	{
	}
	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 10)
	{
		Ar << bDummy << Dummy << Dummy;
	}
	else if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 8)
	{
		Ar << bDummy;
	}
}

//////////////////////////////////////////////////////////// Results

FLidarPointCloudImportResults::FLidarPointCloudImportResults(FThreadSafeBool* bInCancelled /*= nullptr*/, TFunction<void(float)> InProgressCallback /*= TFunction<void(float)>()*/)
	: Bounds(EForceInit::ForceInit)
	, OriginalCoordinates(0)
	, ProgressCallback(MoveTemp(InProgressCallback))
	, bCancelled(bInCancelled)
	, ProgressFrequency(UINT64_MAX)
	, ProgressCounter(0)
	, TotalProgressCounter(0)
	, MaxProgressCounter(0)
{
}

FLidarPointCloudImportResults::FLidarPointCloudImportResults(FThreadSafeBool* bInCancelled, TFunction<void(float)> InProgressCallback, TFunction<void(const FBox& Bounds, FVector)> InInitCallback, TFunction<void(TArray64<FLidarPointCloudPoint>*)> InBufferCallback)
	: FLidarPointCloudImportResults(bInCancelled, MoveTemp(InProgressCallback))
{
	InitCallback = MoveTemp(InInitCallback);
	BufferCallback = MoveTemp(InBufferCallback);
}

void FLidarPointCloudImportResults::InitializeOctree(const FBox& InBounds)
{
	InitCallback(InBounds, OriginalCoordinates);
}

void FLidarPointCloudImportResults::ProcessBuffer(TArray64<FLidarPointCloudPoint>* InPoints)
{
	BufferCallback(InPoints);
	IncrementProgressCounter(InPoints->Num());
}

void FLidarPointCloudImportResults::SetPointCount(const uint64& InTotalPointCount)
{
	SetMaxProgressCounter(InTotalPointCount);
	Points.Empty(InTotalPointCount);
}

void FLidarPointCloudImportResults::AddPoint(const float& X, const float& Y, const float& Z, const float& R, const float& G, const float& B, const float& A /*= 1.0f*/)
{
	Points.Emplace(X, Y, Z, R, G, B, A);
	Bounds += (FVector)Points.Last().Location;
	IncrementProgressCounter(1);
}

void FLidarPointCloudImportResults::AddPoint(const float& X, const float& Y, const float& Z, const float& R, const float& G, const float& B, const float& A, const float& NX, const float& NY, const float& NZ)
{
	Points.Emplace(X, Y, Z, R, G, B, A, NX, NY, NZ);
	Bounds += (FVector)Points.Last().Location;
	IncrementProgressCounter(1);
}

void FLidarPointCloudImportResults::AddPointsBulk(TArray64<FLidarPointCloudPoint>& InPoints)
{
	Points.Append(InPoints);
	IncrementProgressCounter(InPoints.Num());
}

void FLidarPointCloudImportResults::CenterPoints()
{
	// Get the offset value
	FVector CenterOffset = Bounds.GetCenter();

	// Apply it to the points
	for (FLidarPointCloudPoint& Point : Points)
	{
		Point.Location -= (FVector3f)CenterOffset;
	}

	// Account for this in the original coordinates
	OriginalCoordinates += CenterOffset;

	// Shift the bounds
	Bounds = Bounds.ShiftBy(-CenterOffset);
}

void FLidarPointCloudImportResults::SetMaxProgressCounter(uint64 MaxCounter)
{
	MaxProgressCounter = MaxCounter;
	ProgressFrequency = MaxProgressCounter / 100;
}

void FLidarPointCloudImportResults::IncrementProgressCounter(uint64 Increment)
{
	ProgressCounter += Increment;

	if (ProgressCallback && ProgressCounter >= ProgressFrequency)
	{
		TotalProgressCounter += ProgressCounter;
		ProgressCounter = 0;
		ProgressCallback(FMath::Min((double)TotalProgressCounter / MaxProgressCounter, 1.0));
	}
}

//////////////////////////////////////////////////////////// File IO Handler

void FLidarPointCloudFileIOHandler::PrepareImport()
{
	PrecisionCorrectionOffset[0] = 0;
	PrecisionCorrectionOffset[1] = 0;
	PrecisionCorrectionOffset[2] = 0;
	bPrecisionCorrected = false;
}

bool FLidarPointCloudFileIOHandler::ValidateImportSettings(TSharedPtr<FLidarPointCloudImportSettings>& ImportSettings, const FString& Filename)
{
	if (ImportSettings.IsValid())
	{
		if (ImportSettings->IsGeneric())
		{
			// Convert to the specialized settings
			TSharedPtr<FLidarPointCloudImportSettings> NewSettings = GetImportSettings(ImportSettings->Filename);
			NewSettings->bImportAll = ImportSettings->bImportAll;
			ImportSettings = NewSettings;
		}
		else if (!IsSettingsUIDSupported(ImportSettings->GetUID()))
		{
			PC_ERROR("Provided type of ImportSettings does not match the selected importer. Aborting.");
			ImportSettings = nullptr;
		}
	}
	else
	{
		ImportSettings = GetImportSettings(Filename);
	}

	return ImportSettings.IsValid();
}

//////////////////////////////////////////////////////////// File IO

bool ULidarPointCloudFileIO::Import(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults)
{
	bool bSuccess = false;

	FScopeBenchmarkTimer BenchmarkTimer("Importing");
	
	FLidarPointCloudFileIOHandler* Handler = FindHandlerByFilename(Filename);
	if (Handler && Handler->SupportsImport())
	{
		if (Handler->ValidateImportSettings(ImportSettings, Filename))
		{
			Handler->PrepareImport();
			bSuccess = Handler->HandleImport(Filename, ImportSettings, OutImportResults);
		}
	}
	else
	{
		PC_ERROR("No registered importer found for file: %s", *Filename);
	}

	if (!bSuccess)
	{
		BenchmarkTimer.bActive = false;
	}

	return bSuccess;
}

bool ULidarPointCloudFileIO::Export(const FString& Filename, ULidarPointCloud* AssetToExport)
{
	if (AssetToExport)
	{
		FScopeBenchmarkTimer Timer("Exporting");

		FLidarPointCloudFileIOHandler* Handler = ULidarPointCloudFileIO::FindHandlerByFilename(Filename);
		if (Handler && Handler->SupportsExport() && Handler->HandleExport(Filename, AssetToExport))
		{
			return true;
		}
		else
		{
			Timer.bActive = false;
		}
	}

	return false;
}

TSharedPtr<FLidarPointCloudImportSettings> ULidarPointCloudFileIO::GetImportSettings(const FString& Filename)
{
	FLidarPointCloudFileIOHandler *Handler = FindHandlerByFilename(Filename);
	if (Handler && Handler->SupportsImport())
	{
		return Handler->GetImportSettings(Filename);
	}

	return nullptr;
}

TArray<FString> ULidarPointCloudFileIO::GetSupportedImportExtensions()
{
	TArray<FString> Extensions;

	for (const TPair<FString, FLidarPointCloudFileIOHandler*>& Handler : RegisteredHandlers)
	{
		if (Handler.Value->SupportsImport())
		{
			Extensions.Add(Handler.Key);
		}
	}

	return Extensions;
}

TArray<FString> ULidarPointCloudFileIO::GetSupportedExportExtensions()
{
	TArray<FString> Extensions;

	for (const TPair<FString, FLidarPointCloudFileIOHandler*>& Handler : RegisteredHandlers)
	{
		if (Handler.Value->SupportsExport())
		{
			Extensions.Add(Handler.Key);
		}
	}

	return Extensions;
}

void ULidarPointCloudFileIO::RegisterHandler(FLidarPointCloudFileIOHandler* Handler, const TArray<FString>& Extensions)
{
	for (const FString& Extension : Extensions)
	{
		RegisteredHandlers.Emplace(Extension, Handler);
	}
}

FLidarPointCloudFileIOHandler* ULidarPointCloudFileIO::FindHandlerByType(const FString& Type)
{
	FLidarPointCloudFileIOHandler** Handler = RegisteredHandlers.Find(Type);
	return Handler ? *Handler : nullptr;
}

void ULidarPointCloudFileIO::SerializeImportSettings(FArchive& Ar, TSharedPtr<FLidarPointCloudImportSettings>& ImportSettings)
{
	if (Ar.IsLoading())
	{
		FString FilePath;
		Ar << FilePath;

		// If there are no ImportSettings data, do not try to read anything
		if (FilePath.IsEmpty())
		{
			return;
		}

		FLidarPointCloudFileIOHandler* Handler = FindHandlerByFilename(FilePath);
		
		// The importer for this file format is no longer available - no way to proceed
		check(Handler);
		
		ImportSettings = Handler->GetImportSettings(FilePath);
		ImportSettings->Serialize(Ar);
	}
	else
	{
		if (ImportSettings.IsValid())
		{
			Ar << ImportSettings->Filename;
			ImportSettings->Serialize(Ar);
		}
		else
		{
			// If the ImportSettings is invalid, write 0-length FString to indicate it for loading
			FString FilePath = "";
			Ar << FilePath;
		}
	}
}

bool ULidarPointCloudFileIO::FileSupportsConcurrentInsertion(const FString& Filename)
{
	FLidarPointCloudFileIOHandler* Handler = FindHandlerByFilename(Filename);
	return Handler && Handler->SupportsImport() && Handler->SupportsConcurrentInsertion(Filename);
}

ULidarPointCloudFileIO::ULidarPointCloudFileIO(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = ULidarPointCloud::StaticClass();
	PreferredFormatIndex = 0;

	// Requested by UExporter
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		FormatExtension.Append(ULidarPointCloudFileIO::GetSupportedExportExtensions());
		for (int32 i = 0; i < FormatExtension.Num(); i++)
		{
			FormatDescription.Add(TEXT("Point Cloud"));
		}
	}
}

bool ULidarPointCloudFileIO::SupportsObject(UObject* Object) const
{
	bool bSupportsObject = false;

	// Fail, if no exporters are registered
	if (Super::SupportsObject(Object) && GetSupportedExportExtensions().Num() > 0)
	{
		ULidarPointCloud* PointCloud = Cast<ULidarPointCloud>(Object);
		if (PointCloud)
		{
			bSupportsObject = PointCloud->GetNumPoints() > 0;
		}
	}

	return bSupportsObject;
}

bool ULidarPointCloudFileIO::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex /*= 0*/, uint32 PortFlags /*= 0*/)
{
	Export(CurrentFilename, Cast<ULidarPointCloud>(Object));

	// Return false to avoid overwriting the data
	return false;
}
