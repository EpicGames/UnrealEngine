// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/LidarPointCloudFileIO_ASCII.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloud.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"

#define MAX_BUFFER_SIZE 52428800LL

// #todo: Simplify those
#define PC_STREAMTEXTFILE_OFFSET(Offset)															\
	{																								\
		uint32 ReadSize = FMath::Min(RemainingSize, MAX_BUFFER_SIZE - Offset);						\
		DataPtr = Data + Offset;																	\
		Reader->Serialize(DataPtr, ReadSize);														\
		RemainingLoadedSize = ReadSize;																\
		RemainingSize -= ReadSize;																	\
	}

#define PC_STREAMTEXTFILE if (--RemainingLoadedSize <= 0 && RemainingSize > 0) { PC_STREAMTEXTFILE_OFFSET(0) }
#define PC_STREAMTEXTFILE_WITHCHECK																	\
	if (--RemainingLoadedSize <= 0 && RemainingSize > 0)											\
	{																								\
		uint32 OldBufferSize = DataPtr - (uint8*)ChunkStart;										\
																									\
		if (OldBufferSize > 0)																		\
		{																							\
			FMemory::Memcpy(Data, ChunkStart, OldBufferSize);										\
			ChunkStart = (char*)Data;																\
		}																							\
																									\
		PC_STREAMTEXTFILE_OFFSET(OldBufferSize)														\
	}

TSharedPtr<FLidarPointCloudImportSettings_ASCII> MakeImportSettings(const FString& Filename, const FVector2D& RGBRange, const FLidarPointCloudImportSettings_ASCII_Columns& Columns)
{
	TSharedPtr<FLidarPointCloudImportSettings_ASCII> ImportSettings = MakeShared<FLidarPointCloudImportSettings_ASCII>(Filename);
	ImportSettings->RGBRange = RGBRange;
	ImportSettings->SelectedColumns = {
		FMath::Max(-1, Columns.LocationX),
		FMath::Max(-1, Columns.LocationY),
		FMath::Max(-1, Columns.LocationZ),
		FMath::Max(-1, Columns.Red),
		FMath::Max(-1, Columns.Green),
		FMath::Max(-1, Columns.Blue),
		FMath::Max(-1, Columns.Intensity),
		FMath::Max(-1, Columns.NormalX),
		FMath::Max(-1, Columns.NormalY),
		FMath::Max(-1, Columns.NormalZ)};

	return ImportSettings;
}

bool ULidarPointCloudFileIO_ASCII::HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults &OutImportResults)
{
	if (!ValidateImportSettings(ImportSettings, Filename))
	{
		return false;
	}

	FLidarPointCloudImportSettings_ASCII* Settings = (FLidarPointCloudImportSettings_ASCII*)ImportSettings.Get();

	// Modified FFileHelper::LoadFileToString, FFileHelper::LoadFileToStringArray and String::ParseIntoArray
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*Filename));
	if (Reader)
	{
		TArray<int32> SelectedColumns = Settings->SelectedColumns;
		const FVector2D IntensityRange = Settings->ReadFileMinMaxColumns({ SelectedColumns[6] }, true);
		FVector2D RGBRange = Settings->RGBRange;

		// If the range has not been set, set it now
		if (FMath::IsNearlyZero(RGBRange.X) && FMath::IsNearlyZero(RGBRange.Y))
		{
			RGBRange = Settings->ReadFileMinMaxColumns({ SelectedColumns[3], SelectedColumns[4], SelectedColumns[5] }, true);
		}

		const float RGBMulti = 1 / (RGBRange.Y - RGBRange.X);
		const float IntensityMulti = 1 / (IntensityRange.Y - IntensityRange.X);

		// Flags which columns should have data assigned and used
		TArray<bool> IsColumnPopulated;
		IsColumnPopulated.AddZeroed(SelectedColumns.Num());

		OutImportResults.SetPointCount(Settings->EstimatedPointCount);
		OutImportResults.ClassificationsImported.Empty();
		OutImportResults.ClassificationsImported.Add(0);

		uint8 LoadedColumns = 0;

		int32 LargestColumnIndex = -1;
		TArray<int32> ColumnUsageCount;
		TArray<TArray<int32>> LookupTable;

		// Build lookup table
		{
			// Determine the largest index out of all selected columns
			for (int32 SelectedColumnIndex : SelectedColumns)
			{
				LargestColumnIndex = FMath::Max(LargestColumnIndex, SelectedColumnIndex);
			}

			// Only build the lookup table if there are columns assigned
			if (LargestColumnIndex > -1)
			{
				// Reserve necessary space
				ColumnUsageCount.AddDefaulted(LargestColumnIndex + 1);

				for (int32 ColumnIndex = 0; ColumnIndex <= LargestColumnIndex; ++ColumnIndex)
				{
					ColumnUsageCount[ColumnIndex] = 0;

					// Scans selected columns and count the number of times each index is used
					for (int32 SelectedColumnIndex : SelectedColumns)
					{
						if (ColumnIndex == SelectedColumnIndex)
						{
							ColumnUsageCount[ColumnIndex]++;
						}
					}
				}

				// Reserve necessary space
				LookupTable.AddDefaulted(LargestColumnIndex + 1);

				for (int32 ColumnIndex = 0; ColumnIndex <= LargestColumnIndex; ++ColumnIndex)
				{
					// Reserve necessary space
					LookupTable[ColumnIndex].AddDefaulted(ColumnUsageCount[ColumnIndex]);

					// Scans selected columns and assigns them as data sources
					for (int32 AssociatedDataSourceIndex = 0, UsageIndex = 0; AssociatedDataSourceIndex < SelectedColumns.Num(); ++AssociatedDataSourceIndex)
					{
						if (SelectedColumns[AssociatedDataSourceIndex] == ColumnIndex)
						{
							LookupTable[ColumnIndex][UsageIndex++] = AssociatedDataSourceIndex;
						}
					}
				}
			}
		}

		// Read and process the data, if the lookup table exists
		if(LookupTable.Num() > 0)
		{
			int64 TotalSize = Reader->TotalSize();
			int64 RemainingSize = TotalSize;
			int64 RemainingLoadedSize = 0;

			const float ImportScale = GetDefault<ULidarPointCloudSettings>()->ImportScale;

			bool bFirstPointSet = false;

			// Prepare pointers
			uint8 *Data = (uint8*)FMemory::Malloc(MAX_BUFFER_SIZE);
			uint8 *DataPtr = nullptr;

			PC_STREAMTEXTFILE;

			TArray<double> TempDoubles;
			TempDoubles.AddZeroed(SelectedColumns.Num());
			int32 CurrentColumnIndex = 0;
			int64 LineIndex = 0;
			bool bEndOfLine = false;
			while (RemainingLoadedSize > 0 && !OutImportResults.IsCancelled())
			{
				const char* ChunkStart = (char*)DataPtr;
				while (RemainingLoadedSize > 0 && *DataPtr != '\r' && *DataPtr != '\n' && *DataPtr != Settings->Delimiter[0])
				{
					DataPtr++;
					PC_STREAMTEXTFILE_WITHCHECK;
				}

				// Don't parse until the header is skipped
				if (LineIndex >= Settings->LinesToSkip)
				{
					// Number of data slots associated with this column
					int32 DataAssociationCount = CurrentColumnIndex <= LargestColumnIndex ? ColumnUsageCount[CurrentColumnIndex] : 0;

					if (DataAssociationCount)
					{
						double value = atof(ChunkStart);

						for (int32 UsageIndex = 0; UsageIndex < DataAssociationCount; ++UsageIndex)
						{
							const int32 AssociatedDataSourceIndex = LookupTable[CurrentColumnIndex][UsageIndex];

							IsColumnPopulated[AssociatedDataSourceIndex] = true;
							TempDoubles[AssociatedDataSourceIndex] = value;
						}

						LoadedColumns += DataAssociationCount;
					}

					++CurrentColumnIndex;
				}

				if (*DataPtr == Settings->Delimiter[0])
				{
					DataPtr++;
					PC_STREAMTEXTFILE;
				}
				if (*DataPtr == '\r')
				{
					DataPtr++;
					PC_STREAMTEXTFILE;
					bEndOfLine = true;
				}
				if (*DataPtr == '\n')
				{
					DataPtr++;
					PC_STREAMTEXTFILE;
					bEndOfLine = true;
				}
				if (*DataPtr == 0)
				{
					bEndOfLine = true;
				}

				if (bEndOfLine)
				{
					// There is no point in loading data with just 1 column
					if (LoadedColumns > 1)
					{
						// Convert to UU and flip Y axis
						FVector Location(TempDoubles[0] * ImportScale, -TempDoubles[1] * ImportScale, TempDoubles[2] * ImportScale);

						if (!bFirstPointSet)
						{
							OutImportResults.OriginalCoordinates = Location;
							bFirstPointSet = true;
						}

						// Shift to protect from precision loss
						Location -= OutImportResults.OriginalCoordinates;

						const float X = IsColumnPopulated[0] ? Location.X : 0;
						const float Y = IsColumnPopulated[1] ? Location.Y : 0;
						const float Z = IsColumnPopulated[2] ? Location.Z : 0;
						const float R = IsColumnPopulated[3] ? FMath::Clamp((float(TempDoubles[3]) - RGBRange.X) * RGBMulti, 0.0f, 1.0f) : 1;
						const float G = IsColumnPopulated[4] ? FMath::Clamp((float(TempDoubles[4]) - RGBRange.X) * RGBMulti, 0.0f, 1.0f) : 1;
						const float B = IsColumnPopulated[5] ? FMath::Clamp((float(TempDoubles[5]) - RGBRange.X) * RGBMulti, 0.0f, 1.0f) : 1;
						const float A = IsColumnPopulated[6] ? FMath::Clamp((float(TempDoubles[6]) - IntensityRange.X) * IntensityMulti, 0.0f, 1.0f) : 1;

						// If normals are assigned, use the appropriate constructor
						if (IsColumnPopulated[7] && IsColumnPopulated[8] && IsColumnPopulated[9])
						{
							OutImportResults.AddPoint(X, Y, Z, R, G, B, A, TempDoubles[7], TempDoubles[8], TempDoubles[9]);
						}
						else
						{
							OutImportResults.AddPoint(X, Y, Z, R, G, B, A);
						}
					}

					CurrentColumnIndex = 0;
					++LineIndex;
					bEndOfLine = false;
					LoadedColumns = 0;
				}
			}

			FMemory::Free(Data);
		}

		Reader->Close();

		return !OutImportResults.IsCancelled();
	}

	return false;
}

bool ULidarPointCloudFileIO_ASCII::HandleExport(const FString& Filename, class ULidarPointCloud* PointCloud)
{
	// Store the point count
	FFileHelper::SaveStringToFile(FString::Printf(TEXT("%lld\n"), PointCloud->GetNumPoints()), *Filename);

	const float ExportScale = GetDefault<ULidarPointCloudSettings>()->ExportScale;

	const FVector LocationOffset = PointCloud->LocationOffset;

	TArray<FString> Lines;
	Lines.Reserve(5000000);
	PointCloud->Octree.GetPointsAsCopiesInBatches([&LocationOffset, &ExportScale, Filename, &Lines](TSharedPtr<TArray64<FLidarPointCloudPoint>> Points)
	{
		Lines.Reset();

		for (FLidarPointCloudPoint* Data = Points->GetData(), *DestEnd = Data + Points->Num(); Data != DestEnd; ++Data)
		{
			const FVector Location = (LocationOffset + (FVector)Data->Location) * ExportScale;
			const FVector Normal = (FVector)Data->Normal.ToVector();
			Lines.Emplace(FString::Printf(TEXT("%f,%f,%f,%d,%d,%d,%d,%f,%f,%f"), Location.X, -Location.Y, Location.Z, Data->Color.R, Data->Color.G, Data->Color.B, Data->Color.A, Normal.X, Normal.Y, Normal.Z));
		}

		FFileHelper::SaveStringArrayToFile(Lines, *Filename, FFileHelper::EEncodingOptions::ForceAnsi, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	}, 5000000, false);

	return true;
}

void ULidarPointCloudFileIO_ASCII::CreatePointCloudFromFile(UObject* WorldContextObject, const FString& Filename, bool bUseAsync, FVector2D RGBRange, FLidarPointCloudImportSettings_ASCII_Columns Columns, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud)
{
	ULidarPointCloudBlueprintLibrary::CreatePointCloudFromFile(WorldContextObject, Filename, bUseAsync, LatentInfo, MakeImportSettings(Filename, RGBRange, Columns), AsyncMode, Progress, PointCloud);
}

ULidarPointCloud* ULidarPointCloudFileIO_ASCII::CreatePointCloudFromFile(const FString& Filename, const FLidarPointCloudAsyncParameters& AsyncParameters, const FVector2D& RGBRange, const FLidarPointCloudImportSettings_ASCII_Columns& Columns)
{
	return ULidarPointCloud::CreateFromFile(Filename, AsyncParameters, MakeImportSettings(Filename, RGBRange, Columns));
}

FLidarPointCloudImportSettings_ASCII::FLidarPointCloudImportSettings_ASCII(const FString& Filename)
	: FLidarPointCloudImportSettings(Filename)
	, LinesToSkip(0)
	, EstimatedPointCount(0)
	, RGBRange(FVector2D::ZeroVector)
{
	// Initialize all 10 columns as -1 (none)
	for (int32 i = 0; i < 10; i++)
	{
		SelectedColumns.Add(-1);
	}

	ReadFileHeader(Filename);
}

bool FLidarPointCloudImportSettings_ASCII::IsFileCompatible(const FString& InFilename) const
{
	FLidarPointCloudImportSettings_ASCII OtherFile(InFilename);
	return (OtherFile.LinesToSkip == LinesToSkip) && (OtherFile.Columns.Num() == Columns.Num());
}

void FLidarPointCloudImportSettings_ASCII::Serialize(FArchive& Ar)
{
	FLidarPointCloudImportSettings::Serialize(Ar);

	// Used for backwards compatibility
	int32 Dummy32;

	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 18)
	{
		Ar << LinesToSkip << Delimiter << Columns << SelectedColumns << RGBRange;
	}
	else if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 10)
	{
		Ar << LinesToSkip << Dummy32 << Delimiter << Columns << SelectedColumns << RGBRange;
	}
	else
	{
		Ar << LinesToSkip << Dummy32 << Delimiter << Columns << SelectedColumns << RGBRange << Dummy32 << Dummy32;
	}
}

void FLidarPointCloudImportSettings_ASCII::ReadFileHeader(const FString& InFilename)
{
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*InFilename));
	if (Reader)
	{
		int64 TotalSize = Reader->TotalSize();
		if (TotalSize > 0)
		{
			Filename = FPaths::IsRelative(InFilename) ? FPaths::ConvertRelativePathToFull(InFilename) : InFilename;

			int64 Size = FMath::Min(TotalSize, (int64)204800);

			uint8* Data = (uint8*)FMemory::Malloc(Size + 1);
			Reader->Serialize(Data, Size);
			Reader->Close();
			Reader = nullptr;
			uint8* DataPtr = Data;

			Data[Size] = 0;
			LinesToSkip = 0;

			FString Line;
			uint8* LineStart = nullptr;

			while (*DataPtr != 0)
			{
				LineStart = DataPtr;

				while (*DataPtr != 0 && *DataPtr != '\r' && *DataPtr != '\n')
				{
					DataPtr++;
				}

				FFileHelper::BufferToString(Line, LineStart, DataPtr - LineStart);

				// Determine Delimiter
				if (Line.Contains(","))
				{
					Delimiter = ",";
				}
				else if (Line.Contains(" "))
				{
					Delimiter = " ";
				}
				else if (Line.Contains("\t"))
				{
					Delimiter = "\t";
				}

				Line.ParseIntoArray(Columns, *Delimiter);

				if (Columns.Num() >= 3)
				{
					// Skip this line, if it is a text-based header
					for (int32 i = 0; i < Columns.Num(); ++i)
					{
						if (!FCString::IsNumeric(*Columns[i]))
						{
							LinesToSkip++;
							break;
						}
					}

					break;
				}

				LinesToSkip++;

				if (*DataPtr == '\r')
				{
					DataPtr++;
				}
				if (*DataPtr == '\n')
				{
					DataPtr++;
				}
			}

			LineStart = nullptr;

			// Attempt to assign named columns
			bool bNamedColumns = false;
			for (int32 i = 0; i < Columns.Num(); i++)
			{
				FString ColName = *Columns[i];

				// Clean the name
				ColName = ColName.Replace(TEXT("\\"), TEXT("")).Replace(TEXT("/"), TEXT(""));

				bool bNamedColumn = true;

				if (ColName.Equals("x", ESearchCase::IgnoreCase))
				{
					SelectedColumns[0] = i;
				}
				else if (ColName.Equals("y", ESearchCase::IgnoreCase))
				{
					SelectedColumns[1] = i;
				}
				else if (ColName.Equals("z", ESearchCase::IgnoreCase))
				{
					SelectedColumns[2] = i;
				}
				else if (ColName.Equals("r", ESearchCase::IgnoreCase) || ColName.Contains("red"))
				{
					SelectedColumns[3] = i;
				}
				else if (ColName.Equals("g", ESearchCase::IgnoreCase) || ColName.Contains("green"))
				{
					SelectedColumns[4] = i;
				}
				else if (ColName.Equals("b", ESearchCase::IgnoreCase) || ColName.Contains("blue"))
				{
					SelectedColumns[5] = i;
				}
				else if (ColName.Equals("intensity", ESearchCase::IgnoreCase) || ColName.Contains("alpha"))
				{
					SelectedColumns[6] = i;
				}
				else
				{
					bNamedColumn = false;
				}

				bNamedColumns |= bNamedColumn;
			}

			// If named columns were not found, fallback to index-based assignment
			if (!bNamedColumns)
			{
				// Assume at least XYZ
				if (Columns.Num() >= 3)
				{
					SelectedColumns[0] = 0;
					SelectedColumns[1] = 1;
					SelectedColumns[2] = 2;

					// Assume XYZ I
					if (Columns.Num() == 4)
					{
						SelectedColumns[6] = 3;
					}
					// Assume XYZ I RGB
					else if (Columns.Num() == 7)
					{
						SelectedColumns[3] = 4;
						SelectedColumns[4] = 5;
						SelectedColumns[5] = 6;
						SelectedColumns[6] = 3;
					}
					// Assume XYZ RGB
					else if (Columns.Num() >= 6)
					{
						SelectedColumns[3] = 3;
						SelectedColumns[4] = 4;
						SelectedColumns[5] = 5;
					}
				}
			}

			// Estimate number of points
			uint64 ContentSize = TotalSize;
			if (LinesToSkip > 0)
			{
				Size -= Line.Len() + 1;
				ContentSize -= Line.Len() + 1;

				if (*DataPtr == '\r')
				{
					DataPtr++;
				}
				if (*DataPtr == '\n')
				{
					DataPtr++;
				}
			}

			int64 LineCount = 0;
			int32 Count = 0;
			while (*DataPtr != 0)
			{
				while (*DataPtr != 0 && *DataPtr != '\r' && *DataPtr != '\n')
				{
					Count++;
					DataPtr++;
				}

				if (Count > 0)
				{
					LineCount++;
					Count = 0;
				}

				if (*DataPtr == '\r')
				{
					DataPtr++;
				}
				if (*DataPtr == '\n')
				{
					DataPtr++;
				}
			}

			EstimatedPointCount = ContentSize * LineCount / Size;

			FMemory::Free(Data);
			DataPtr = nullptr;
		}
	}
}

FVector2D FLidarPointCloudImportSettings_ASCII::ReadFileMinMaxColumns(TArray<int32> ColumnsToScan, bool bBestMatch)
{
	// Check if we actually have any columns to search for
	bool bHasValidColumns = false;
	for (int32 i = 0; i < ColumnsToScan.Num(); i++)
	{
		if (ColumnsToScan[i] > -1)
		{
			bHasValidColumns = true;
			break;
		}
	}

	if (!bHasValidColumns)
	{
		return FVector2D::ZeroVector;
	}

	FVector2D Result = FVector2D::ZeroVector;

	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*Filename));
	if (Reader)
	{
		Result = FVector2D(FLT_MAX, -FLT_MAX);
		float Tmp = 0;

		const int64 TotalSize = Reader->TotalSize();
		int64 RemainingSize = TotalSize;
		int64 RemainingLoadedSize = 0;

		// Prepare pointers
		uint8 *Data = (uint8*)FMemory::Malloc(MAX_BUFFER_SIZE);
		uint8 *DataPtr = nullptr;

		PC_STREAMTEXTFILE;

		uint8 Iterator = 0;
		bool bEndOfLine = false;
		int64 LineNumber = 0;
		while (RemainingLoadedSize > 0 && LineNumber < 100000)
		{
			const char* ChunkStart = (char*)DataPtr;
			while (RemainingLoadedSize > 0 && *DataPtr != '\r' && *DataPtr != '\n' && *DataPtr != Delimiter[0])
			{
				DataPtr++;
				PC_STREAMTEXTFILE_WITHCHECK;
			}

			// Make sure we skip the header
			if (LineNumber >= LinesToSkip && ColumnsToScan.Contains(Iterator++))
			{
				Tmp = (float)atof(ChunkStart);
				Result.X = FMath::Min(Result.X, Tmp);
				Result.Y = FMath::Max(Result.Y, Tmp);
			}

			if (*DataPtr == Delimiter[0])
			{
				DataPtr++;
				PC_STREAMTEXTFILE;
			}
			if (*DataPtr == '\r')
			{
				DataPtr++;
				PC_STREAMTEXTFILE;
				bEndOfLine = true;
			}
			if (*DataPtr == '\n')
			{
				DataPtr++;
				PC_STREAMTEXTFILE;
				bEndOfLine = true;
			}

			if (bEndOfLine)
			{
				Iterator = 0;
				bEndOfLine = false;
				LineNumber++;
			}
		}

		Reader->Close();
		Reader = nullptr;
		FMemory::Free(Data);
		Data = nullptr;
		DataPtr = nullptr;
	}

	if (bBestMatch)
	{
		if (Result.X < 0)
		{
			if (Result.Y > 32768) { return Result; }	// Not really sure the scale here, return original
			else if (Result.Y > 512) { return FVector2D(-32767, 32768); }
			else if (Result.Y > 128) { return FVector2D(-511, 512); }
			else { return FVector2D(-127, 128); }
		}
		else
		{
			if (Result.Y > 65535) { return Result; }	// Not really sure the scale here, return original
			else if (Result.Y > 4095) { return FVector2D(0, 65535); }
			else if (Result.Y > 1023) { return FVector2D(0, 4095); }
			else if (Result.Y > 255) { return FVector2D(0, 1023); }
			else if (Result.Y > 1) { return FVector2D(0, 255); }
			else { return FVector2D(0, 1); }
		}
	}
	else
	{
		return Result;
	}
}

#if WITH_EDITOR
#define LOCTEXT_NAMESPACE "PointCloudImportSettings"

#define ADD_DROPDOWN(Index, Label) 	+ SHorizontalBox::Slot()	\
									.Padding(2)	\
									[	\
										SNew(SComboBox<TSharedPtr<FString>>)	\
										.OptionsSource(&Options)	\
										.InitiallySelectedItem(Options[SelectedColumns[Index] + 1])	\
										.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NewItem, ESelectInfo::Type SelectInfo) { SelectedColumns[Index] = Options.IndexOfByKey(NewItem) - 1; })	\
										.OnGenerateWidget_Lambda([this](TSharedPtr<FString> Item) { return HandleGenerateWidget(*Item); })	\
										[	\
											HandleGenerateWidget(Label)	\
										]	\
									]

#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"

TSharedPtr<SWidget> FLidarPointCloudImportSettings_ASCII::GetWidget()
{	
	Options.Empty();
	Options.Add(MakeShared<FString>(TEXT("- NONE -")));

	for (int32 i = 0; i < Columns.Num(); i++)
	{
		Options.Add(MakeShared<FString>(*Columns[i]));
	}		

	TSharedPtr<SVerticalBox> SettingsWidget;
	
	// Initialize widget structure
	{
		SAssignNew(SettingsWidget, SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(STextBlock).Text(FText::Format(LOCTEXT("PointCount", "Estimated Point Count: {0}"), FText::AsNumber(EstimatedPointCount))).Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2, 10, 2, 2)
		[
			SNew(STextBlock).Text(LOCTEXT("AssignColumns", "Specify data source for each property.")).Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			ADD_DROPDOWN(0, "Loc X")
			ADD_DROPDOWN(1, "Loc Y")
			ADD_DROPDOWN(2, "Loc Z")
			ADD_DROPDOWN(3, "Red")
			ADD_DROPDOWN(4, "Green")
			ADD_DROPDOWN(5, "Blue")
			ADD_DROPDOWN(6, "Intensity")
			ADD_DROPDOWN(7, "Norm X")
			ADD_DROPDOWN(8, "Norm Y")
			ADD_DROPDOWN(9, "Norm Z")
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2, 10, 2, 2)
		[
			SNew(STextBlock).Text(LOCTEXT("RGBRange", "Specify color range to use for normalization. Leave both at 0 for auto-matching.")).Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.65f)
			[
				SNew(SHorizontalBox)
				.ToolTipText(LOCTEXT("PointCloudImportSettings_RGBRange_ToolTip", "Values outside the specified range will be clamped to either MIN or MAX value.\nLeaving both values at 0 will auto-determine the range."))
				+ SHorizontalBox::Slot()
				.Padding(2)
				.FillWidth(0.5f)
				[
					SAssignNew(RGBRangeMin, SSpinBox<float>)
					.MinValue(-FLT_MAX)
					.MaxValue(FLT_MAX)
					.Value(RGBRange.X)
					.OnValueChanged_Lambda([this](float NewValue) { RGBRange.X = NewValue; })
				]
				+ SHorizontalBox::Slot()
				.Padding(2)
				.FillWidth(0.5f)
				[
					SAssignNew(RGBRangeMax, SSpinBox<float>)
					.MinValue(-FLT_MAX)
					.MaxValue(FLT_MAX)
					.Value(RGBRange.Y)
					.OnValueChanged_Lambda([this](float NewValue) { RGBRange.Y = NewValue; })
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.125f)
			.Padding(2)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("GetMinMaxScal", "Scan"))
				.ToolTipText(LOCTEXT("GetMinMaxScan_ToolTip", "This will scan through the file to determine the min and max values."))
				.OnClicked_Lambda([this]()
				{
					RGBRange = ReadFileMinMaxColumns({ SelectedColumns[3], SelectedColumns[4], SelectedColumns[5] }, false);
					RGBRangeMin->SetValue(RGBRange.X);
					RGBRangeMax->SetValue(RGBRange.Y);
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.225f)
			.Padding(2)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("GetMinMaxMatch", "Scan & Match"))
				.ToolTipText(LOCTEXT("GetMinMaxMatch_ToolTip", "This will scan through the file to determine the min and max values. Best matching range will be chosen."))
				.OnClicked_Lambda([this]()
				{
					RGBRange = ReadFileMinMaxColumns({ SelectedColumns[3], SelectedColumns[4], SelectedColumns[5] }, true);
					RGBRangeMin->SetValue(RGBRange.X);
					RGBRangeMax->SetValue(RGBRange.Y);
					return FReply::Handled();
				})
			]
		];
	}

	return SettingsWidget;
}

TSharedRef<SWidget> FLidarPointCloudImportSettings_ASCII::HandleGenerateWidget(FString Item) const
{
	TSharedPtr<STextBlock> NewItem = SNew(STextBlock).Text(FText::FromString(Item)).Font(IDetailLayoutBuilder::GetDetailFont());
	NewItem->SetMargin(FMargin(2, 2, 5, 2));
	return NewItem.ToSharedRef();
}

#undef ADD_DROPDOWN
#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR

#undef PC_STREAMTEXTFILE
#undef PC_STREAMTEXTFILE_OFFSET
#undef PC_STREAMTEXTFILE_WITHCHECK