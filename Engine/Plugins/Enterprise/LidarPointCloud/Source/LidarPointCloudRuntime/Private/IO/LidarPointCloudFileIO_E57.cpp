// Copyright Epic Games, Inc. All Rights Reserved.

#if LIBE57SUPPORTED

#include "IO/LidarPointCloudFileIO_E57.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloud.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Async/Async.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeCounter64.h"
#include "Interfaces/IPluginManager.h"

#pragma pack(push)
#pragma pack(1)
struct FE57Point
{
	FVector Location;
	FColor Color;
	FVector3f Normal;
};
#pragma pack(pop)

FCriticalSection _E57DLLLock;

#define LOADFUNC(Return,Name,...) typedef Return(*Name##_def)(__VA_ARGS__); static Name##_def h##Name = (Name##_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT(#Name))

struct FE57PoseTransform
{
	FQuat Rotation;
	FVector Translation;

	FE57PoseTransform()
		: Rotation(FQuat::Identity)
		, Translation(FVector::ZeroVector)
	{
	}
};

class FE57Wrapper
{
	typedef TFunction<bool(uint32, uint32, bool, bool, FE57PoseTransform)> FBatchCallback;

	void* ImageFile;

public:
	FE57Wrapper() : ImageFile(nullptr) {}
	~FE57Wrapper()
	{
		CloseFile();
	}

	bool OpenFile(const FString& Filename)
	{
		LOADFUNC(void*, OpenFile, const char*);
		ImageFile = hOpenFile(TCHAR_TO_ANSI(*Filename.Replace(TEXT("\\"), TEXT("/"))));
		return !!ImageFile;
	}

	void CloseFile()
	{		
		if (ImageFile)
		{
			LOADFUNC(void, CloseFile, void*&);
			hCloseFile(ImageFile);
		}
	}

	int64 GetNumPoints()
	{
		if (ImageFile)
		{
			LOADFUNC(int64, GetNumPoints, void*);
			return hGetNumPoints(ImageFile);
		}
		else
		{
			return -1;
		}
	}

	uint32 GetNumScans()
	{
		if (ImageFile)
		{
			LOADFUNC(uint32, GetNumScans, void*);
			return hGetNumScans(ImageFile);
		}
		else
		{
			return -1;
		}
	}

	FBox GetBounds(uint32 ScanID)
	{
		if (ImageFile)
		{
			LOADFUNC(FBox, GetBounds, void*, uint32);
			return hGetBounds(ImageFile, ScanID);
		}
		else
		{
			return FBox(EForceInit::ForceInit);
		}
	}

	FE57PoseTransform GetPoseTransform(uint32 ScanID)
	{
		if (ImageFile)
		{
			LOADFUNC(FE57PoseTransform, GetPoseTransform, void*, uint32);
			return hGetPoseTransform(ImageFile, ScanID);
		}
		else
		{
			return FE57PoseTransform();
		}
	}

	void GetFirstLocation(FVector& OutLocation, FE57PoseTransform& OutTransform)
	{
		if (ImageFile)
		{
			LOADFUNC(void, GetFirstLocation, void*, FVector&, FE57PoseTransform&);
			return hGetFirstLocation(ImageFile, OutLocation, OutTransform);
		}
	}

	void ReadPoints(void* Buffer, uint32 BatchSize, FBatchCallback InBatchCallback)
	{		
		if (ImageFile)
		{
			LOADFUNC(void, ReadPoints, void*, void*, uint32, void*, void*);
			hReadPoints(ImageFile, Buffer, BatchSize, &ReadPointsCallbackFunc, &InBatchCallback);
		}
	}

private:
	static bool ReadPointsCallbackFunc(uint32 ScanID, uint32 NumPointsRead, bool bHasRGB, bool bHasIntensity, FE57PoseTransform Transform, FBatchCallback* Callback)
	{
		return (*Callback)(ScanID, NumPointsRead, bHasRGB, bHasIntensity, Transform);
	}

	void* GetDLLHandle()
	{
		static void* v_dllHandle = nullptr;

		if (!v_dllHandle)
		{
			FString DllDirectory = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("LidarPointCloud"))->GetBaseDir(), TEXT("Source/ThirdParty/LibE57"));
#if PLATFORM_WINDOWS
			FPlatformProcess::GetDllHandle(*FPaths::Combine(DllDirectory, TEXT("Win64/xerces-c_3_2.dll")));
			v_dllHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(DllDirectory, TEXT("Win64/E57UE4.dll")));
#endif
		}
		
		return v_dllHandle;
	}
};

#undef LOADFUNC

bool ULidarPointCloudFileIO_E57::HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults)
{
	FScopeLock DllLock(&_E57DLLLock);

	FE57Wrapper Reader;
	if (!Reader.OpenFile(Filename))
	{
		PC_ERROR("ULidarPointCloudFileIO_E57: Cannot create reader for file: '%s'", *Filename);
		return false;
	}

	const int64 TotalPointsToRead = Reader.GetNumPoints();

	if (TotalPointsToRead < 1)
	{
		PC_ERROR("ULidarPointCloudFileIO_E57: Incorrect number of points: %lld", TotalPointsToRead);
		return false;
	}

	OutImportResults.SetMaxProgressCounter(TotalPointsToRead);

	const float ImportScale = GetDefault<ULidarPointCloudSettings>()->ImportScale;

	// Clear any existing data
	OutImportResults.Bounds.Init();
	OutImportResults.Points.Empty();

	// Get scan offset
	{
		FE57PoseTransform Transform;
		Reader.GetFirstLocation(OutImportResults.OriginalCoordinates, Transform);
		
		OutImportResults.OriginalCoordinates = (Transform.Rotation.RotateVector(OutImportResults.OriginalCoordinates) + Transform.Translation) * ImportScale;
		OutImportResults.OriginalCoordinates.Y = -OutImportResults.OriginalCoordinates.Y;
	}

	FCriticalSection SyncLock;
	TArray<TFuture<void>> ThreadResults;

	constexpr int64 MaxBufferSize = 2000000LL;
	const uint32 MaxPointsToRead = FMath::Min(TotalPointsToRead, MaxBufferSize);

	static FLidarPointCloudDataBufferManager BufferManager(MaxBufferSize * sizeof(FE57Point));
	FLidarPointCloudDataBuffer* Buffer = BufferManager.GetFreeBuffer();

	// Read actual data
	Reader.ReadPoints(Buffer->GetData(), MaxPointsToRead,
		[Buffer, &OutImportResults, ImportScale, &SyncLock, &ThreadResults](uint32 ScanID, uint32 NumPointsRead, bool bHasRGB, bool bHasIntensity, FE57PoseTransform Transform)
		{
			FLidarPointCloudDataBuffer* ThreadBuffer = BufferManager.GetFreeBuffer();
			FMemory::Memcpy(ThreadBuffer->GetData(), Buffer->GetData(), NumPointsRead * sizeof(FE57Point));

			ThreadResults.Add(Async(EAsyncExecution::Thread, [NumPointsRead, bHasRGB, bHasIntensity, Transform, &OutImportResults, ImportScale, &SyncLock, ThreadBuffer]{
				TArray64<FLidarPointCloudPoint> Points;
				Points.Reserve(NumPointsRead);
				FBox Bounds(EForceInit::ForceInit);

				for (FE57Point* Data = (FE57Point*)ThreadBuffer->GetData(), *DataEnd = Data + NumPointsRead; Data != DataEnd; ++Data)
				{
					// Apply transformations
					Data->Location = (Transform.Rotation.RotateVector(Data->Location) + Transform.Translation) * ImportScale;
					Data->Location.Y = -Data->Location.Y;

					// Shift to protect from precision loss
					Data->Location -= OutImportResults.OriginalCoordinates;
					
					Bounds += Data->Location;

					if (!bHasRGB)
					{
						Data->Color.R = Data->Color.G = Data->Color.B = 255;
					}

					if(!bHasIntensity)
					{
						Data->Color.A = 255;
					}

					Points.Emplace((FVector3f)Data->Location, Data->Color, true, 0, FLidarPointCloudNormal(Data->Normal));
				}

				ThreadBuffer->MarkAsFree();

				// Data Sync
				FScopeLock Sync(&SyncLock);

				OutImportResults.AddPointsBulk(Points);
				OutImportResults.Bounds += Bounds;
			}));

			return !OutImportResults.IsCancelled();
		});

	Buffer->MarkAsFree();

	// Sync threads
	for (const TFuture<void>& ThreadResult : ThreadResults)
	{
		ThreadResult.Get();
	}

	// Make sure to progress the counter to the end before returning
	OutImportResults.IncrementProgressCounter(TotalPointsToRead);

	return !OutImportResults.IsCancelled();
}

#endif