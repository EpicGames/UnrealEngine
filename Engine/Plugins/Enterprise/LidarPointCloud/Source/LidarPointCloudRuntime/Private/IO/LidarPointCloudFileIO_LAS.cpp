// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/LidarPointCloudFileIO_LAS.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloud.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Async/Async.h"
#include "HAL/ThreadSafeCounter64.h"
#include "Interfaces/IPluginManager.h"

/** LAS - Public Header Block as per LAS Specification v.1.4 - R14 */
struct FLidarPointCloudFileIO_LAS_PublicHeaderBlock
{
	struct LASZipVLR
	{
		uint16 reserved;
		char user_id[16];
		uint16 record_id;
		uint16 record_length_after_header;
		char description[32];
		uint8* data;
	};

	uint16 FileSourceID;
	uint16 GlobalEncoding;
	uint32 ProjectID_GUIDData1;
	uint16 ProjectID_GUIDData2;
	uint16 ProjectID_GUIDData3;
	uint8 ProjectID_GUIDData4[8];
	uint8 VersionMajor;
	uint8 VersionMinor;
	char SystemIdentifier[32];
	char GeneratingSoftware[32];
	uint16 FileCreationDayofYear;
	uint16 FileCreationYear;
	uint16 HeaderSize;
	uint32 OffsetToPointData;
	uint32 NumberOfVLRs;
	uint8 PointDataRecordFormat;
	uint16 PointDataRecordLength;
	uint32 LegacyNumberOfPointRecords;
	uint32 LegacyNumberOfPointsByReturn[5];
	FVector ScaleFactor;
	FVector Offset;
	FVector Min;
	FVector Max;

	/** Added in 1.3, extra 8 bytes */
	uint64 StartOfWaveformDataPacketRecord;

	/** Added in 1.4, extra 140 bytes */
	uint64 StartOfFirstEVLR;
	uint32 NumberOfEVLRs;
	uint64 NumberOfPointRecords;
	uint64 NumberOfPointsByReturn[15];

	// optional
	uint32 user_data_in_header_size;
	uint8* user_data_in_header;

	// optional VLRs
	LASZipVLR* vlrs;

	// optional
	uint32 user_data_after_header_size;
	uint8* user_data_after_header;

	FLidarPointCloudFileIO_LAS_PublicHeaderBlock()
	{
		FMemory::Memzero(this, sizeof(FLidarPointCloudFileIO_LAS_PublicHeaderBlock));

		VersionMajor = 1;
		VersionMinor = 2;

		const char* _SystemIdentifier = "Unreal Engine 4";
		FMemory::Memcpy(SystemIdentifier, _SystemIdentifier, 15);

		const char* _GeneratingSoftware = "LiDAR Point Cloud Plugin";
		FMemory::Memcpy(GeneratingSoftware, _GeneratingSoftware, 24);

		FileCreationDayofYear = FDateTime::Now().GetDayOfYear();
		FileCreationYear = FDateTime::Now().GetYear();

		HeaderSize = OffsetToPointData = 227;

		PointDataRecordFormat = 2;
		PointDataRecordLength = 26;
	}

	bool IsValid()
	{
		return VersionMajor > 0 || VersionMinor > 0;
	}

	uint64 GetNumberOfPoints()
	{
		return VersionMinor < 4 ? LegacyNumberOfPointRecords : NumberOfPointRecords;
	}

	void SetNumberOfPoints(uint64 NewNumberOfPoints)
	{
		LegacyNumberOfPointRecords = NumberOfPointRecords = NewNumberOfPoints;

		// We need to switch to 1.4 spec to support 64-bit indexing
		if (NewNumberOfPoints > UINT32_MAX)
		{
			VersionMinor = 4;
			HeaderSize = OffsetToPointData = 375;
		}
	}

	void SetBounds(const FVector& InMin, const FVector& InMax)
	{
		Min = InMin;
		Max = InMax;
		Offset = InMin;

		FVector Size = Max - Min;
		ScaleFactor = FVector(FMath::Pow(2.0, FMath::CeilToInt(FMath::Log2(Size.X)) - 31), FMath::Pow(2.0, FMath::CeilToInt(FMath::Log2(Size.Y)) - 31), FMath::Pow(2.0, FMath::CeilToInt(FMath::Log2(Size.Z)) - 31));
	}

	bool HasValidBounds()
	{
		return (Max.X > Min.X || (Max.X == Min.X && Max.X != 0)) && (Max.Y > Min.Y || (Max.Y == Min.Y && Max.Y != 0)) && (Max.Z > Min.Z || (Max.Z == Min.Z && Max.Z != 0));
	}

	bool HasRGB()
	{
		switch (PointDataRecordFormat)
		{
		case 2:
		case 3:
		case 5:
		case 7:
		case 8:
		case 10:
			return true;
		}

		return false;
	}

	int32 GetRGBOffset()
	{
		switch (PointDataRecordFormat)
		{
		case 2:
			return 20;

		case 3:
		case 5:
			return 28;

		case 7:
		case 8:
		case 10:
			return 30;
		}

		return 0;
	}

	int32 GetClassificationOffset()
	{
		return PointDataRecordFormat < 6 ? 15 : 16;
	}
};

FArchive& operator<<(FArchive& Ar, FLidarPointCloudFileIO_LAS_PublicHeaderBlock& Header)
{
	// char[] { 'L', 'A', 'S', 'F' } read as uint32 - 1179861324
	uint32 FileSignature = 1179861324;
	Ar << FileSignature;
	Ar << Header.FileSourceID;
	Ar << Header.GlobalEncoding;
	Ar << Header.ProjectID_GUIDData1 << Header.ProjectID_GUIDData2 << Header.ProjectID_GUIDData3;
	Ar.Serialize(&Header.ProjectID_GUIDData4, 8);
	Ar << Header.VersionMajor << Header.VersionMinor;
	Ar.Serialize(&Header.SystemIdentifier, 32);
	Ar.Serialize(&Header.GeneratingSoftware, 32);
	Ar << Header.FileCreationDayofYear << Header.FileCreationYear;
	Ar << Header.HeaderSize << Header.OffsetToPointData;
	Ar << Header.NumberOfVLRs;
	Ar << Header.PointDataRecordFormat;
	Ar << Header.PointDataRecordLength;
	Ar << Header.LegacyNumberOfPointRecords;
	Ar.Serialize(&Header.LegacyNumberOfPointsByReturn, 20);
	Ar << Header.ScaleFactor << Header.Offset;
	Ar << Header.Max << Header.Min;

	// Use legacy bounds order, if needed
	if (Header.VersionMinor < 4 || !Header.HasValidBounds())
	{
		FVector Min = Header.Min;
		FVector Max = Header.Max;

		Header.Max.X = Max.X;
		Header.Max.Y = Max.Z;
		Header.Max.Z = Min.Y;
		Header.Min.X = Max.Y;
		Header.Min.Y = Min.X;
		Header.Min.Z = Min.Z;
	}

	if (Header.VersionMinor >= 3)
	{
		Ar << Header.StartOfWaveformDataPacketRecord;
	}

	if (Header.VersionMinor >= 4)
	{
		Ar << Header.StartOfFirstEVLR << Header.NumberOfEVLRs << Header.NumberOfPointRecords;
		Ar.Serialize(&Header.NumberOfPointsByReturn, 120);
	}

	return Ar;
}

#pragma pack(push)
#pragma pack(1)
/** Contains the RGB extension structure data compatible with all formats */
struct FLidarPointCloudFileIO_LAS_PointDataRecordFormatCommonRGB
{
	uint16 Red;
	uint16 Green;
	uint16 Blue;
};

/** LAS - Point Data Record Formats as per LAS Specification v.1.4 - R14 */
struct FLidarPointCloudFileIO_LAS_PointDataRecordFormat0
{
	FIntVector Location;
	uint16 Intensity;
	uint8 ReturnNumber : 3;
	uint8 NumberOfReturns : 3;
	uint8 ScanDirectionFlag : 1;
	uint8 EdgeOfFlightLine : 1;
	uint8 Classification;
	int8 ScanAngle;
	uint8 UserData;
	uint16 PointSourceID;
};
struct FLidarPointCloudFileIO_LAS_PointDataRecordFormat2 : FLidarPointCloudFileIO_LAS_PointDataRecordFormat0, FLidarPointCloudFileIO_LAS_PointDataRecordFormatCommonRGB { };
#pragma pack(pop)

#if LASZIPSUPPORTED
struct FLASZipPoint
{
	FIntVector Location;
	uint16 Intensity;
	uint8 ReturnNumber : 3;
	uint8 NumberOfReturns : 3;
	uint8 ScanDirectionFlag : 1;
	uint8 EdgeOfFlightLine : 1;
	uint8 Classification : 5;
	uint8 synthetic_flag : 1;
	uint8 keypoint_flag : 1;
	uint8 withheld_flag : 1;
	int8 scan_angle_rank;
	uint8 user_data;
	uint16 point_source_ID;

	// LAS 1.4 only
	int16 extended_scan_angle;
	uint8 extended_point_type : 2;
	uint8 extended_scanner_channel : 2;
	uint8 extended_classification_flags : 4;
	uint8 extended_classification;
	uint8 extended_return_number : 4;
	uint8 extended_number_of_returns : 4;

	// for 8 byte alignment of the GPS time
	uint8 dummy[7];

	double gps_time;
	uint16 rgb[4];
	uint8 wave_packet[29];

	int32 num_extra_bytes;
	uint8* extra_bytes;
};

class FLASZipWrapper
{
public:
	enum OpenFileMode
	{
		Reader,
		Writer
	};

private:
	typedef int32(*laszip_create_def)(void** pointer);
	typedef int32(*laszip_close_reader_def)(void* pointer);
	typedef int32(*laszip_close_writer_def)(void* pointer);
	typedef int32(*laszip_destroy_def)(void* pointer);
	typedef int32(*laszip_open_reader_def)(void* pointer, const char* file_name, int32* is_compressed);
	typedef int32(*laszip_open_writer_def)(void* pointer, const char* file_name, int32 is_compressed);
	typedef int32(*laszip_get_header_pointer_def)(void* pointer, FLidarPointCloudFileIO_LAS_PublicHeaderBlock** header);
	typedef int32(*laszip_set_header_def)(void* pointer, const FLidarPointCloudFileIO_LAS_PublicHeaderBlock* header);
	typedef int32(*laszip_get_point_pointer_def)(void* pointer, FLASZipPoint** point_pointer);
	typedef int32(*laszip_set_point_def)(void* pointer, const FLASZipPoint* point_pointer);
	typedef int32(*laszip_seek_point_def)(void* pointer, int64 index);
	typedef int32(*laszip_read_point_def)(void* pointer);
	typedef int32(*laszip_write_point_def)(void* pointer);
	typedef int32(*laszip_get_error_def)(void* pointer, char** out_error);

	void* laszip_ptr;
	FLASZipPoint* point;
	OpenFileMode Mode;

public:
	FLASZipWrapper()
		: laszip_ptr(nullptr)
		, point(nullptr)
	{
		static laszip_create_def laszip_create = (laszip_create_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_create"));
		laszip_create(&laszip_ptr);
	}
	~FLASZipWrapper()
	{
		if (!!laszip_ptr)
		{
			if (Mode == Reader)
			{
				static laszip_close_reader_def laszip_close_reader = (laszip_close_reader_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_close_reader"));
				laszip_close_reader(laszip_ptr);
			}
			else
			{
				static laszip_close_writer_def laszip_close_writer = (laszip_close_writer_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_close_writer"));
				laszip_close_writer(laszip_ptr);
			}

			static laszip_destroy_def laszip_destroy = (laszip_destroy_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_destroy"));
			laszip_destroy(laszip_ptr);
		}
	}

	bool OpenFile(const FString& Filename, OpenFileMode InMode)
	{
		FString ConvertedPath = Filename.Replace(TEXT("\\"), TEXT("/"));
		Mode = InMode;
		if (Mode == Reader)
		{
			static laszip_open_reader_def laszip_open_reader = (laszip_open_reader_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_open_reader"));
			int32 is_compressed = 1;
			return !laszip_open_reader(laszip_ptr, TCHAR_TO_ANSI(*ConvertedPath), &is_compressed);
		}
		else
		{
			static laszip_open_writer_def laszip_open_writer = (laszip_open_writer_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_open_writer"));
			return !laszip_open_writer(laszip_ptr, TCHAR_TO_ANSI(*ConvertedPath), 1);
		}
	}

	bool ReadHeader(FLidarPointCloudFileIO_LAS_PublicHeaderBlock* &OutHeader)
	{
		static laszip_get_header_pointer_def laszip_get_header_pointer = (laszip_get_header_pointer_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_get_header_pointer"));
		bool bSuccess = !laszip_get_header_pointer(laszip_ptr, &OutHeader);
		
		if (bSuccess)
		{
			// LASzip keeps the bounds in 1.2 spec
			FVector Min = OutHeader->Min;
			FVector Max = OutHeader->Max;
			OutHeader->Min.X = Min.Y;
			OutHeader->Min.Y = Max.X;
			OutHeader->Min.Z = Max.Z;
			OutHeader->Max.X = Min.X;
			OutHeader->Max.Y = Min.Z;
			OutHeader->Max.Z = Max.Y;
		}

		return bSuccess;
	}

	bool ReadHeader(FLidarPointCloudFileIO_LAS_PublicHeaderBlock& OutHeader)
	{
		FLidarPointCloudFileIO_LAS_PublicHeaderBlock* Header;
		if (ReadHeader(Header))
		{
			OutHeader = *Header;
			return true;
		}

		return false;
	}

	bool SetHeader(FLidarPointCloudFileIO_LAS_PublicHeaderBlock& Header)
	{
		static laszip_set_header_def laszip_set_header = (laszip_set_header_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_set_header"));

		// LASzip keeps the bounds in 1.2 spec
		FVector Min = Header.Min;
		FVector Max = Header.Max;
		Header.Min.X = Max.X;
		Header.Min.Y = Min.X;
		Header.Min.Z = Max.Y;
		Header.Max.X = Min.Y;
		Header.Max.Y = Max.Z;
		Header.Max.Z = Min.Z;

		return !laszip_set_header(laszip_ptr, &Header);
	}

	bool ReadNextPoint(FLASZipPoint& OutPoint)
	{		
		static laszip_get_point_pointer_def laszip_get_point_pointer = (laszip_get_point_pointer_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_get_point_pointer"));
		static laszip_read_point_def laszip_read_point = (laszip_read_point_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_read_point"));

		if (!point)
		{
			laszip_get_point_pointer(laszip_ptr, &point);
		}

		if (!laszip_read_point(laszip_ptr))
		{
			OutPoint = *point;
			return true;
		}

		return  false;
	}

	bool WritePoint(FLASZipPoint& Point)
	{
		static laszip_set_point_def laszip_set_point = (laszip_set_point_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_set_point"));
		static laszip_write_point_def laszip_write_point = (laszip_write_point_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_write_point"));

		return  !laszip_set_point(laszip_ptr, &Point) && !laszip_write_point(laszip_ptr);
	}

	void Seek(int64 Position)
	{
		static laszip_seek_point_def laszip_seek_point = (laszip_seek_point_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_seek_point"));
		laszip_seek_point(laszip_ptr, Position);
	}

	FString GetError()
	{
		static laszip_get_error_def laszip_get_error = (laszip_get_error_def)FPlatformProcess::GetDllExport(GetDLLHandle(), TEXT("laszip_get_error"));
		
		FString Error;

		char* _Error;
		if (!laszip_get_error(laszip_ptr, &_Error))
		{
			Error = FString(_Error);
		}

		return Error;
	}

private:
	void* GetDLLHandle()
	{
		static void* v_dllHandle = nullptr;

		if (!v_dllHandle)
		{
			FString DllDirectory = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("LidarPointCloud"))->GetBaseDir(), TEXT("Source/ThirdParty/LasZip"));
#if PLATFORM_MAC
			v_dllHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(DllDirectory, TEXT("Mac/laszip.dylib")));
#elif PLATFORM_WINDOWS
			v_dllHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(DllDirectory, TEXT("Win64/laszip.dll")));
#endif
		}

		return v_dllHandle;
	}
};
#endif

bool ULidarPointCloudFileIO_LAS::HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults)
{
	const FString Extension = FPaths::GetExtension(Filename).ToLower();

	if (Extension.Equals("las"))
	{
		return HandleImportLAS(Filename, OutImportResults);
	}
#if LASZIPSUPPORTED
	else if (Extension.Equals("laz"))
	{
		return HandleImportLAZ(Filename, OutImportResults);
	}
#endif
	else
	{
		return false;
	}
}

bool ULidarPointCloudFileIO_LAS::HandleImportLAS(const FString& Filename, FLidarPointCloudImportResults& OutImportResults)
{
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*Filename));
	if (!Reader)
	{
		PC_ERROR("LiDAR LAS Importer: Cannot create reader for file: '%s'", *Filename);
		return false;
	}

	int64 TotalSize = Reader->TotalSize();

	if (TotalSize < 227)
	{
		PC_ERROR("LiDAR LAS Importer: Incorrect file size: %lld", TotalSize);
		return false;
	}

	FLidarPointCloudFileIO_LAS_PublicHeaderBlock Header;
	(*Reader) << Header;

	if (!Header.IsValid())
	{
		PC_ERROR("LiDAR LAS Importer: Incorrect header");
		return false;
	}

	int64 TotalPointsToRead = Header.GetNumberOfPoints();

	if (TotalPointsToRead < 1)
	{
		PC_ERROR("LiDAR LAS Importer: Incorrect number of points: %lld", TotalPointsToRead);
		return false;
	}

	OutImportResults.SetMaxProgressCounter(TotalPointsToRead);

	const bool bUseConcurrentImport = SupportsConcurrentInsertion(Filename);

	bool bHasIntensityData = false;
	uint8 IntensityBitShift = 0;
	uint8 RGBBitShift = 0;

	const bool bHasRGB = Header.HasRGB();
	const int32 RGBOffset = Header.GetRGBOffset();
	const int32 ClassificationOffset = Header.GetClassificationOffset();

	const float ImportScale = GetDefault<ULidarPointCloudSettings>()->ImportScale;

	// Calculate max buffer size
	const int64 MaxPointsToRead = FMath::Min(TotalPointsToRead, 2000000LL);

	// Detect bit depth
	{
		uint16 MaxIntensity = 0;
		uint16 MaxRGB = 0;
		bool bFirstPointSet = false;

		// Set the correct position for the reader
		Reader->Seek(Header.OffsetToPointData);

		// Calculate the amount of data to read
		int64 PointsToRead = FMath::Min(1000000LL, TotalPointsToRead);

		// Create data buffer
		TArray<uint8> Data;
		Data.AddUninitialized(PointsToRead * Header.PointDataRecordLength);

		// Read the data
		Reader->Serialize(Data.GetData(), Data.Num());

		for (uint8* DataPtr = Data.GetData(), *DataEnd = DataPtr + Data.Num(); DataPtr != DataEnd; DataPtr += Header.PointDataRecordLength)
		{
			const FLidarPointCloudFileIO_LAS_PointDataRecordFormat0* Record = (FLidarPointCloudFileIO_LAS_PointDataRecordFormat0*)DataPtr;

			MaxIntensity = FMath::Max(MaxIntensity, Record->Intensity);

			if (bHasRGB)
			{
				const FLidarPointCloudFileIO_LAS_PointDataRecordFormatCommonRGB* RecordRGB = (FLidarPointCloudFileIO_LAS_PointDataRecordFormatCommonRGB*)(DataPtr + RGBOffset);
				MaxRGB = FMath::Max(MaxRGB, FMath::Max3(RecordRGB->Red, RecordRGB->Green, RecordRGB->Blue));
			}

			if (!bFirstPointSet)
			{
				OutImportResults.OriginalCoordinates = FVector(Header.ScaleFactor.X * Record->Location.X + Header.Offset.X,
																Header.ScaleFactor.Y * Record->Location.Y + Header.Offset.Y,
																Header.ScaleFactor.Z * Record->Location.Z + Header.Offset.Z) * ImportScale;
				OutImportResults.OriginalCoordinates.Y = -OutImportResults.OriginalCoordinates.Y;
				bFirstPointSet = true;
			}
		}

		bHasIntensityData = MaxIntensity > 0;
		IntensityBitShift = MaxIntensity > 255 ? MaxIntensity > 4095 ? 8 : 4 : 0;
		RGBBitShift = MaxRGB > 255 ? MaxRGB > 4095 ? 8 : 4 : 0;
	}

	if (bUseConcurrentImport)
	{
		FBox Bounds(Header.Min * ImportScale, Header.Max * ImportScale);

		// Flip Y
		const double Tmp = Bounds.Min.Y;
		Bounds.Min.Y = -Bounds.Max.Y;
		Bounds.Max.Y = -Tmp;
		
		OutImportResults.InitializeOctree(Bounds);
	}

	// Read Data
	{
		// Set the correct position for the reader
		Reader->Seek(Header.OffsetToPointData);

		int64 PointsRead = 0;

		// Clear any existing data
		OutImportResults.Points.Empty(bUseConcurrentImport ? 0 : TotalPointsToRead);
		OutImportResults.ClassificationsImported.Empty();

		// Multi-threading
		FCriticalSection PointsLock;
		TArray<TFuture<void>> ThreadResults;
		FLidarPointCloudDataBufferManager BufferManager(MaxPointsToRead * Header.PointDataRecordLength, FPlatformMisc::NumberOfCoresIncludingHyperthreads() - 1);

		// Stream the data
		while (PointsRead < TotalPointsToRead && !OutImportResults.IsCancelled())
		{
			FLidarPointCloudDataBuffer* Buffer = BufferManager.GetFreeBuffer();

			// Data should never be null
			check(Buffer->GetData());

			// Calculate the amount of data to read
			int64 PointsToRead = FMath::Min(MaxPointsToRead, TotalPointsToRead - PointsRead);

			// Read the data
			Reader->Serialize(Buffer->GetData(), PointsToRead * Header.PointDataRecordLength);

			ThreadResults.Add(Async(EAsyncExecution::Thread, [PointsToRead, Buffer, &ImportScale, &bHasIntensityData, IntensityBitShift, RGBBitShift, &Header, &OutImportResults, &PointsLock, bHasRGB, RGBOffset, ClassificationOffset, bUseConcurrentImport]
				{
					uint8* Data = Buffer->GetData();

					TArray64<FLidarPointCloudPoint> Points;
					Points.Reserve(PointsToRead);

					FBox _Bounds(EForceInit::ForceInit);

					TArray<uint8> Classifications;

					// Parse the data
					for (int64 j = 0; j < PointsToRead && !OutImportResults.IsCancelled(); j++)
					{
						FLidarPointCloudFileIO_LAS_PointDataRecordFormat0* Record = (FLidarPointCloudFileIO_LAS_PointDataRecordFormat0*)Data;

						const uint8 Intensity = (bHasIntensityData ? (Record->Intensity >> IntensityBitShift) : 255);
						const uint8 Classification = *(Data + ClassificationOffset);

						Classifications.AddUnique(Classification);

						// Calculate the actual location of the point, convert to UU and flip the Y axis
						FVector Location = FVector(Header.ScaleFactor.X * Record->Location.X + Header.Offset.X,
												Header.ScaleFactor.Y * Record->Location.Y + Header.Offset.Y,
												Header.ScaleFactor.Z * Record->Location.Z + Header.Offset.Z) * ImportScale;
						Location.Y = -Location.Y;

						// Shift to protect from precision loss
						Location -= OutImportResults.OriginalCoordinates;

						// Convert location to floats
						const FVector ProcessedLocation = Location;

						_Bounds += ProcessedLocation;

						if (bHasRGB)
						{
							FLidarPointCloudFileIO_LAS_PointDataRecordFormatCommonRGB* RecordRGB = (FLidarPointCloudFileIO_LAS_PointDataRecordFormatCommonRGB*)(Data + RGBOffset);
							Points.Emplace((FVector3f)ProcessedLocation, RecordRGB->Red >> RGBBitShift, RecordRGB->Green >> RGBBitShift, RecordRGB->Blue >> RGBBitShift, Intensity, Classification);
						}
						else
						{
							Points.Emplace((FVector3f)ProcessedLocation, 255, 255, 255, Intensity, Classification);
						}

						// Increment the data pointer
						Data += Header.PointDataRecordLength;
					}

					Buffer->MarkAsFree();

					if (bUseConcurrentImport)
					{
						OutImportResults.ProcessBuffer(&Points);
					}
					else
					{
						// Data Sync
						FScopeLock Lock(&PointsLock);

						OutImportResults.AddPointsBulk(Points);
						OutImportResults.Bounds += _Bounds;
					}

					// Data Sync
					FScopeLock Lock(&PointsLock);

					for (uint8& Classification : Classifications)
					{
						OutImportResults.ClassificationsImported.AddUnique(Classification);
					}
				}));

			PointsRead += PointsToRead;
		}

		// Sync threads
		for (const TFuture<void>& ThreadResult : ThreadResults)
		{
			ThreadResult.Get();
		}

		// Make sure to progress the counter to the end before returning
		OutImportResults.IncrementProgressCounter(TotalPointsToRead);
	}

	// Free memory
	Reader->Close();

	return !OutImportResults.IsCancelled();
}

#if LASZIPSUPPORTED
bool ULidarPointCloudFileIO_LAS::HandleImportLAZ(const FString& Filename, FLidarPointCloudImportResults& OutImportResults)
{
	TArray<FLASZipWrapper> LASZipWrappers;
	LASZipWrappers.AddDefaulted(1);
	TArray<FLASZipPoint> Points;
	Points.AddDefaulted(1);

	FLASZipWrapper& LASZipWrapper = LASZipWrappers[0];
	if (!LASZipWrapper.OpenFile(Filename, FLASZipWrapper::Reader))
	{
		PC_ERROR("LiDAR LAZ Importer: Cannot create reader for file: '%s'", *Filename);
		return false;
	}

	FLidarPointCloudFileIO_LAS_PublicHeaderBlock Header;
	LASZipWrapper.ReadHeader(Header);

	if (!Header.IsValid())
	{
		PC_ERROR("LiDAR LAZ Importer: Incorrect header");
		return false;
	}

	int64 TotalPointsToRead = Header.GetNumberOfPoints();

	if (TotalPointsToRead < 1)
	{
		PC_ERROR("LiDAR LAZ Importer: Incorrect number of points: %lld", TotalPointsToRead);
		return false;
	}

	OutImportResults.SetMaxProgressCounter(TotalPointsToRead);

	const bool bUseConcurrentImport = SupportsConcurrentInsertion(Filename);

	const bool bHasRGB = Header.HasRGB();
	bool bHasIntensityData = false;
	uint8 IntensityBitShift = 0;
	uint8 RGBBitShift = 0;

	const float ImportScale = GetDefault<ULidarPointCloudSettings>()->ImportScale;

	// Detect bit depth
	{
		FLASZipPoint& Point = Points[0];

		bool bFirstPointSet = false;
		uint16 MaxIntensity = 0;
		uint16 MaxRGB = 0;

		// Calculate the amount of data to read
		int64 PointsToRead = FMath::Min(1000000LL, TotalPointsToRead);

		while (LASZipWrapper.ReadNextPoint(Point) && !OutImportResults.IsCancelled() && (PointsToRead--) > 0)
		{
			MaxIntensity = FMath::Max(MaxIntensity, Point.Intensity);

			if (bHasRGB)
			{
				MaxRGB = FMath::Max(MaxRGB, FMath::Max3(Point.rgb[0], Point.rgb[1], Point.rgb[2]));
			}

			if (!bFirstPointSet)
			{
				OutImportResults.OriginalCoordinates = FVector(Header.ScaleFactor.X * Point.Location.X + Header.Offset.X,
															Header.ScaleFactor.Y * Point.Location.Y + Header.Offset.Y,
															Header.ScaleFactor.Z * Point.Location.Z + Header.Offset.Z) * ImportScale;
				OutImportResults.OriginalCoordinates.Y = -OutImportResults.OriginalCoordinates.Y;
				bFirstPointSet = true;
			}
		}

		bHasIntensityData = MaxIntensity > 0;
		IntensityBitShift = MaxIntensity > 255 ? MaxIntensity > 4095 ? 8 : 4 : 0;
		RGBBitShift = MaxRGB > 255 ? MaxRGB > 4095 ? 8 : 4 : 0;
	}

	// Initialize Octree
	if (bUseConcurrentImport)
	{
		FBox Bounds(Header.Min * ImportScale, Header.Max * ImportScale);

		// Flip Y
		const double Tmp = Bounds.Min.Y;
		Bounds.Min.Y = -Bounds.Max.Y;
		Bounds.Max.Y = -Tmp;
		
		OutImportResults.InitializeOctree(Bounds);
	}

	// Read Data
	{
		// Clear any existing data
		OutImportResults.Bounds.Init();
		OutImportResults.Points.Empty(TotalPointsToRead);
		OutImportResults.ClassificationsImported.Empty();

		FCriticalSection PointsLock;
		TArray<TFuture<void>> ThreadResults;

		// This is to avoid spinning too many threads
		const int32 NumThreads = FMath::Min(TotalPointsToRead / 3000000 + 1, FPlatformMisc::NumberOfCoresIncludingHyperthreads() - 1LL);
		const int64 Batch = TotalPointsToRead / NumThreads + 1;

		// Use -1 to account for the default one already created
		LASZipWrappers.AddDefaulted(NumThreads - 1);
		Points.AddDefaulted(NumThreads - 1);

		for (int32 t = 0; t < NumThreads; ++t)
		{
			FLASZipWrapper& Wrapper = LASZipWrappers[t];
			FLASZipPoint& Point = Points[t];

			if (t > 0)
			{
				Wrapper.OpenFile(Filename, FLASZipWrapper::Reader);
			}

			const int64 StartIdx = Batch * t;
			const int64 EndIdx = t == NumThreads - 1 ? TotalPointsToRead : StartIdx + Batch;

			Wrapper.Seek(StartIdx);

			ThreadResults.Add(Async(EAsyncExecution::Thread, [t, StartIdx, EndIdx, &Wrapper, &Point, &OutImportResults, bHasIntensityData, IntensityBitShift, bHasRGB, RGBBitShift, ImportScale, Header, &PointsLock, bUseConcurrentImport]
			{
				// Local batching to avoid spiking RAM
				const int64 BatchSize = 1000000;

				TArray64<FLidarPointCloudPoint> _Points;
				_Points.Reserve(BatchSize);
				TArray<uint8> Classifications;

				FBox _Bounds(EForceInit::ForceInit);

				for (int64 i = StartIdx; i < EndIdx && !OutImportResults.IsCancelled(); ++i)
				{
					Wrapper.ReadNextPoint(Point);

					const uint8 Classification = Point.Classification;
					Classifications.AddUnique(Classification);

					// Calculate the actual location of the point, convert to UU and flip the Y axis
					FVector Location = FVector(Header.ScaleFactor.X * Point.Location.X + Header.Offset.X,
											Header.ScaleFactor.Y * Point.Location.Y + Header.Offset.Y,
											Header.ScaleFactor.Z * Point.Location.Z + Header.Offset.Z) * ImportScale;
					Location.Y = -Location.Y;

					// Shift to protect from precision loss
					Location -= OutImportResults.OriginalCoordinates;

					const FVector ProcessedLocation = Location;

					const uint8 Intensity = bHasIntensityData ? Point.Intensity >> IntensityBitShift : 255;

					if (bHasRGB)
					{
						_Points.Emplace((FVector3f)ProcessedLocation, Point.rgb[0] >> RGBBitShift, Point.rgb[1] >> RGBBitShift, Point.rgb[2] >> RGBBitShift, Intensity, Classification);
					}
					else
					{
						_Points.Emplace((FVector3f)ProcessedLocation, 255, 255, 255, Intensity, Classification);
					}

					if (bUseConcurrentImport)
					{
						if (_Points.Num() == BatchSize)
						{
							OutImportResults.ProcessBuffer(&_Points);
							_Points.Reset();
						}
					}
					else
					{
						_Bounds += ProcessedLocation;
					}
				}

				if (bUseConcurrentImport && _Points.Num() > 0)
				{
					OutImportResults.ProcessBuffer(&_Points);
				}

				// Data Sync
				FScopeLock Lock(&PointsLock);

				if (!bUseConcurrentImport)
				{
					OutImportResults.AddPointsBulk(_Points);
					OutImportResults.Bounds += _Bounds;
				}

				for (uint8& Classification : Classifications)
				{
					OutImportResults.ClassificationsImported.AddUnique(Classification);
				}
			}));
		}

		// Sync threads
		for (const TFuture<void>& ThreadResult : ThreadResults)
		{
			ThreadResult.Get();
		}
	}

	return !OutImportResults.IsCancelled();
}
#endif

bool ULidarPointCloudFileIO_LAS::HandleExport(const FString& Filename, ULidarPointCloud* PointCloud)
{
	const FString Extension = FPaths::GetExtension(Filename).ToLower();

	if (Extension.Equals("las"))
	{
		return HandleExportLAS(Filename, PointCloud);
	}
#if LASZIPSUPPORTED
	else if (Extension.Equals("laz"))
	{
		return HandleExportLAZ(Filename, PointCloud);
	}
#endif
	else
	{
		return false;
	}
}

bool ULidarPointCloudFileIO_LAS::HandleExportLAS(const FString& Filename, ULidarPointCloud* PointCloud)
{
	if (FArchive* Ar = IFileManager::Get().CreateFileWriter(*Filename, 0))
	{
		FVector Min = PointCloud->GetBounds().Min;
		FVector Max = PointCloud->GetBounds().Max;

		// Flip Y
		float MaxY = Max.Y;
		Max.Y = -Min.Y;
		Min.Y = -MaxY;

		const float ExportScale = GetDefault<ULidarPointCloudSettings>()->ExportScale;

		// Convert to meters
		Min *= ExportScale;
		Max *= ExportScale;

		FLidarPointCloudFileIO_LAS_PublicHeaderBlock Header;
		Header.SetNumberOfPoints(PointCloud->GetNumPoints());
		Header.SetBounds(Min, Max);

		(*Ar) << Header;

		const FVector Size = Max - Min;
		const FVector ForwardScale(FMath::Pow(2.0, 31 - FMath::CeilToInt(FMath::Log2(Size.X))), FMath::Pow(2.0, 31 - FMath::CeilToInt(FMath::Log2(Size.Y))), FMath::Pow(2.0, 31 - FMath::CeilToInt(FMath::Log2(Size.Z))));
		const FVector LocationOffset = PointCloud->LocationOffset;

		const int32 BatchSize = 500000;
		TArray<FLidarPointCloudFileIO_LAS_PointDataRecordFormat2> PointRecords;
		PointRecords.AddUninitialized(BatchSize);
		FLidarPointCloudFileIO_LAS_PointDataRecordFormat2* PointRecordsPtr = PointRecords.GetData();

		PointCloud->Octree.GetPointsAsCopiesInBatches([&LocationOffset, &ExportScale, &Min, &ForwardScale, &Header, Ar, PointRecordsPtr](TSharedPtr<TArray64<FLidarPointCloudPoint>> Points)
		{
			const int64 Size = Header.PointDataRecordLength * Points->Num();
			FMemory::Memzero(PointRecordsPtr, Size);

			FLidarPointCloudPoint* Data = Points->GetData();
			for (FLidarPointCloudFileIO_LAS_PointDataRecordFormat2* Dest = PointRecordsPtr, *DestEnd = Dest + Points->Num(); Dest != DestEnd; ++Dest, ++Data)
			{
				FVector Location = (LocationOffset + (FVector)Data->Location) * ExportScale;
				Location.Y = -Location.Y;

				Dest->Location = FIntVector(ForwardScale * (Location - Min));
				Dest->Intensity = (Data->Color.A << 8) + Data->Color.A;
				Dest->Red = (Data->Color.R << 8) + Data->Color.R;
				Dest->Green = (Data->Color.G << 8) + Data->Color.G;
				Dest->Blue = (Data->Color.B << 8) + Data->Color.B;
				Dest->Classification = Data->ClassificationID;
			}

			Ar->Serialize(PointRecordsPtr, Size);
		}, BatchSize, false);

		delete Ar;
		return true;
	}

	return false;
}

#if LASZIPSUPPORTED
bool ULidarPointCloudFileIO_LAS::HandleExportLAZ(const FString& Filename, ULidarPointCloud* PointCloud)
{
	FLASZipWrapper LASZipWrapper;

	FVector Min = PointCloud->GetBounds(false).Min;
	FVector Max = PointCloud->GetBounds(false).Max;

	// Flip Y
	float MaxY = Max.Y;
	Max.Y = -Min.Y;
	Min.Y = -MaxY;

	const float ExportScale = GetDefault<ULidarPointCloudSettings>()->ExportScale;

	// Convert to meters
	Min *= ExportScale;
	Max *= ExportScale;

	FLidarPointCloudFileIO_LAS_PublicHeaderBlock Header;
	Header.SetNumberOfPoints(PointCloud->GetNumPoints());
	Header.SetBounds(Min, Max);

	LASZipWrapper.SetHeader(Header);	
	LASZipWrapper.OpenFile(Filename, FLASZipWrapper::Writer);

	FLASZipPoint PointRecord;
	FMemory::Memzero(&PointRecord, sizeof(FLASZipPoint));

	const FVector Size = Max - Min;
	const FVector ForwardScale(FMath::Pow(2.0, 31 - FMath::CeilToInt(FMath::Log2(Size.X))), FMath::Pow(2.0, 31 - FMath::CeilToInt(FMath::Log2(Size.Y))), FMath::Pow(2.0, 31 - FMath::CeilToInt(FMath::Log2(Size.Z))));
	const FVector LocationOffset = PointCloud->LocationOffset;

	PointCloud->ExecuteActionOnAllPoints([&PointRecord, &LASZipWrapper, &LocationOffset, &ExportScale, &Min, &ForwardScale](FLidarPointCloudPoint* Point)
	{
		FVector Location = (LocationOffset + (FVector)Point->Location) * ExportScale;
		Location.Y = -Location.Y;

		PointRecord.Location = FIntVector(ForwardScale * (Location - Min));
		PointRecord.Intensity = Point->Color.A << 8;
		PointRecord.rgb[0] = Point->Color.R << 8;
		PointRecord.rgb[1] = Point->Color.G << 8;
		PointRecord.rgb[2] = Point->Color.B << 8;
		PointRecord.Classification = Point->ClassificationID;

		LASZipWrapper.WritePoint(PointRecord);
	}, false);

	return true;
}
#endif

bool ULidarPointCloudFileIO_LAS::SupportsConcurrentInsertion(const FString& Filename) const
{
	const FString Extension = FPaths::GetExtension(Filename).ToLower();

	if (Extension.Equals("las"))
	{
		TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*Filename));
		if (Reader)
		{
			FLidarPointCloudFileIO_LAS_PublicHeaderBlock Header;
			(*Reader) << Header;
			return Header.HasValidBounds();
		}
	}
#if LASZIPSUPPORTED
	else if (Extension.Equals("laz"))
	{
		FLASZipWrapper LASZipWrapper;
		if (LASZipWrapper.OpenFile(Filename, FLASZipWrapper::Reader))
		{
			FLidarPointCloudFileIO_LAS_PublicHeaderBlock Header;
			LASZipWrapper.ReadHeader(Header);
			return Header.HasValidBounds();
		}
	}
#endif

	return false;
}

//----------------------------------------------------------------

void FLidarPointCloudImportSettings_LAS::Serialize(FArchive& Ar)
{
	FLidarPointCloudImportSettings::Serialize(Ar);

	int32 Dummy32;
	uint8 Dummy;

	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 11)
	{
	}
	else if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 10)
	{
		Ar << Dummy;
	}
	else if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 7)
	{
		Ar << Dummy32 << Dummy32 << Dummy;
	}
}
