// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloud.h"
#include "Framework/Application/SlateApplication.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudActor.h"
#include "LidarPointCloudComponent.h"
#include "IO/LidarPointCloudFileIO.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Serialization/CustomVersion.h"
#include "Misc/ScopeTryLock.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/Engine.h"
#include "LatentActions.h"
#include "PhysicsEngine/BodySetup.h"
#include "Framework/Notifications/NotificationManager.h"
#include "EngineUtils.h"
#include "Components/BrushComponent.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/GCObjectScopeGuard.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/MessageDialog.h"
#endif

#define IS_PROPERTY(Name) PropertyChangedEvent.MemberProperty->GetName().Equals(#Name)

const FGuid ULidarPointCloud::PointCloudFileGUID('P', 'C', 'P', 'F');
const int32 ULidarPointCloud::PointCloudFileVersion(20);
FCustomVersionRegistration PCPFileVersion(ULidarPointCloud::PointCloudFileGUID, ULidarPointCloud::PointCloudFileVersion, TEXT("LiDAR Point Cloud File Version"));

#define LOCTEXT_NAMESPACE "LidarPointCloud"

#if WITH_EDITOR
#include "PackageTools.h"

/* Registers for the package modified callback to catch point clouds that have been saved and automatically reloads the asset to release the bulk data */
class FLidarPackageReloader : public FTickableGameObject
{
	TArray<TWeakObjectPtr<UPackage>> PackagesToReload;

public:
	FLidarPackageReloader() { UPackage::PackageSavedWithContextEvent.AddRaw(this, &FLidarPackageReloader::OnPackageSaved); }
	virtual ~FLidarPackageReloader() { UPackage::PackageSavedWithContextEvent.RemoveAll(this); }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(LidarPackageReloader, STATGROUP_Tickables); }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual bool IsTickableInEditor() const override { return true; }

	virtual void Tick(float DeltaTime) override
	{
		TArray<UPackage*> TopLevelPackages;

		for (TWeakObjectPtr<UPackage> Package : PackagesToReload)
		{
			if (UPackage* PackagePtr = Package.Get())
			{
				TopLevelPackages.Add(PackagePtr);
			}
		}

		if (TopLevelPackages.Num() > 0)
		{
			UPackageTools::ReloadPackages(TopLevelPackages);
		}

		PackagesToReload.Reset();
	}

private:
	void OnPackageSaved(const FString& Filename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
	{
		if (GetDefault<ULidarPointCloudSettings>()->bReleaseAssetAfterSaving)
		{
			if(Package->GetLinker())
			{
				if (Cast<ULidarPointCloud>(Package->FindAssetInPackage()))
				{
					PackagesToReload.Add(Package);
				}
			}
		}
	}
};
#endif

/////////////////////////////////////////////////
// FPointCloudLatentAction

class FPointCloudSimpleLatentAction : public FPendingLatentAction
{
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

public:
	bool bComplete;

	FPointCloudSimpleLatentAction(const FLatentActionInfo& LatentInfo)
		: ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
		, bComplete(false)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		Response.FinishAndTriggerIf(bComplete, ExecutionFunction, OutputLink, CallbackTarget);
	}
};

class FPointCloudLatentAction : public FPendingLatentAction
{
public:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	ELidarPointCloudAsyncMode* Mode = nullptr;

	FPointCloudLatentAction(const FLatentActionInfo& LatentInfo, ELidarPointCloudAsyncMode& Mode)
		: ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
		, Mode(&Mode)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		if (*Mode != ELidarPointCloudAsyncMode::Progress)
		{
			Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
		}
		else
		{
			Response.TriggerLink(ExecutionFunction, OutputLink, CallbackTarget);
		}
	}
};

/////////////////////////////////////////////////
// FLidarPointCloudNotification

/** Wrapper around a NotificationItem to make the notification handling more centralized */
class FLidarPointCloudNotification
{
	/** Stores the pointer to the actual notification item */
	TSharedPtr<SNotificationItem> NotificationItem;

	FString CurrentText;
	int8 CurrentProgress;

public:
	FLidarPointCloudNotification(const FString& Text, TWeakObjectPtr<ULidarPointCloud> Owner, FThreadSafeBool* bCancelPtr, const FString& Icon)
		: CurrentText(Text)
		, CurrentProgress(-1)
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			// Build the notification widget
			FNotificationInfo Info(FText::FromString(CurrentText));
			Info.bFireAndForget = false;
			Info.Image = FSlateStyleRegistry::FindSlateStyle("LidarPointCloudStyle")->GetBrush(*Icon);

			if (ULidarPointCloud* OwnerPtr = Owner.Get())
			{
				const FSoftObjectPath SoftObjectPath(OwnerPtr);

				Info.Hyperlink = FSimpleDelegate::CreateLambda([SoftObjectPath] {
					// Select the cloud in Content Browser when the hyperlink is clicked
					TArray<FAssetData> AssetData;
					AssetData.Add(FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get().GetAssetByObjectPath(SoftObjectPath.GetWithoutSubPath()));
					FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get().SyncBrowserToAssets(AssetData);
					});
				Info.HyperlinkText = FText::FromString(FPaths::GetBaseFilename(SoftObjectPath.ToString()));
			}

			if (bCancelPtr)
			{
				Info.ButtonDetails.Emplace(
					LOCTEXT("OpCancel", "Cancel"),
					LOCTEXT("OpCancelToolTip", "Cancels the point cloud operation in progress."),
					FSimpleDelegate::CreateLambda([bCancelPtr] { *bCancelPtr = true; })
				);
			}

			NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
			if (NotificationItem.IsValid())
			{
				NotificationItem->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
#endif
	}
	~FLidarPointCloudNotification()
	{
		Close(false);
	}

	void SetText(const FString& Text)
	{
#if WITH_EDITOR
		CurrentText = Text;
		UpdateStatus();
#endif
	}
	void SetProgress(int8 Progress)
	{
#if WITH_EDITOR
		CurrentProgress = Progress;
		UpdateStatus();
#endif
	}
	void SetTextWithProgress(const FString& Text, int8 Progress)
	{
#if WITH_EDITOR
		CurrentText = Text;
		CurrentProgress = Progress;
		UpdateStatus();
#endif
	}
	void Close(bool bSuccess)
	{
#if WITH_EDITOR
		if (NotificationItem.IsValid())
		{
			// Do not use fadeout if the engine is shutting down
			if(FSlateApplication::IsInitialized())
			{
				CurrentText.Append(bSuccess ? " Complete" : " Failed");
				CurrentProgress = -1;
				UpdateStatus();
				NotificationItem->SetCompletionState(bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
				NotificationItem->ExpireAndFadeout();
			}
			NotificationItem.Reset();
		}
#endif
	}

private:
	void UpdateStatus()
	{
#if WITH_EDITOR
		if (NotificationItem.IsValid())
		{
			if (IsInGameThread())
			{
				FString Message;

				if (CurrentProgress >= 0)
				{
					Message = FString::Printf(TEXT("%s: %d%%"), *CurrentText, CurrentProgress);
				}
				else
				{
					Message = FString::Printf(TEXT("%s"), *CurrentText);
				}

				NotificationItem->SetText(FText::FromString(Message));
			}
			else
			{
				AsyncTask(ENamedThreads::GameThread, [this] { UpdateStatus(); });
			}
		}
#endif
	}
};

TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> ULidarPointCloud::FLidarPointCloudNotificationManager::Create(const FString& Text, FThreadSafeBool* bCancelPtr, const FString& Icon)
{
	return Notifications[Notifications.Add(MakeShared<FLidarPointCloudNotification, ESPMode::ThreadSafe>(Text, Owner, bCancelPtr, Icon))];
}

void ULidarPointCloud::FLidarPointCloudNotificationManager::CloseAll()
{
	for (TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification : Notifications)
	{
		Notification->Close(false);
	}

	Notifications.Empty();
}

/////////////////////////////////////////////////
// ULidarPointCloud

ULidarPointCloud::ULidarPointCloud()
	: MaxCollisionError(100)
	, NormalsQuality(40)
	, NormalsNoiseTolerance(1)
	, OriginalCoordinates(FVector::ZeroVector)
	, bOptimizedForDynamicData(false)
	, Octree(this)
	, LocationOffset(FVector::ZeroVector)
	, Notifications(this)
	, BodySetup(nullptr)
	, NewBodySetup(nullptr)
	, bCollisionBuildInProgress(false)
{
	// Make sure we are transactional to allow undo redo
	this->SetFlags(RF_Transactional);

#if WITH_EDITOR
	static FLidarPackageReloader LidarPackageReloader;
#endif
}

void ULidarPointCloud::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ULidarPointCloud::Serialize"), STAT_PointCLoud_Serialize, STATGROUP_LoadTime);

	Ar.UsingCustomVersion(PointCloudFileGUID);

	Super::Serialize(Ar);
		
	int32 Version = Ar.CustomVer(PointCloudFileGUID);

	if (Version > 13)
	{
		Ar << BodySetup;

		if (Ar.IsCountingMemory())
		{
			if (BodySetup)
			{
				BodySetup->Serialize(Ar);
			}
		}
	}

	// Make sure to serialize only actual data
	if (Ar.ShouldSkipBulkData() || Ar.IsObjectReferenceCollector() || !Ar.IsPersistent())
	{
		return;
	}
	
	ULidarPointCloudFileIO::SerializeImportSettings(Ar, ImportSettings);

	// Do not save the Octree, if in the middle of processing or the access to the data is blocked
	{
		FScopeTryLock LockProcessing(&ProcessingLock);
		FScopeTryLock LockOctree(&Octree.DataLock);

		bool bValidOctree = LockProcessing.IsLocked() && LockOctree.IsLocked();
		Ar << bValidOctree;
		if (bValidOctree)
		{
			Ar << Octree;
		}
	}
}

void ULidarPointCloud::PostLoad()
{
	Super::PostLoad();

	InitializeCollisionRendering();
}

void ULidarPointCloud::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void ULidarPointCloud::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Context.AddTag(FAssetRegistryTag("PointCount", PointCloudAssetRegistryCache.PointCount, FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("ApproxSize", PointCloudAssetRegistryCache.ApproxSize, FAssetRegistryTag::TT_Dimensional));

	Super::GetAssetRegistryTags(Context);
}

void ULidarPointCloud::BeginDestroy()
{
	Super::BeginDestroy();

	// Cancel async import and wait for it to exit
	bAsyncCancelled = true;
	FScopeLock LockImport(&ProcessingLock);

	// Hide any notifications, if still present
	Notifications.CloseAll();

	// Wait for ongoing data access to finish
	FScopeLock LockOctree(&Octree.DataLock);

	// Release and destroy any collision rendering data, if present
	ReleaseCollisionRendering(true);
}

void ULidarPointCloud::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
	OnPreSaveCleanupEvent.Broadcast();
}

#if WITH_EDITOR
void ULidarPointCloud::ClearAllCachedCookedPlatformData()
{
	if (GetDefault<ULidarPointCloudSettings>()->bReleaseAssetAfterCooking)
	{
		Octree.ReleaseAllNodes(true);
	}
}

void ULidarPointCloud::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty)
	{		
		if (IS_PROPERTY(SourcePath))
		{
			SetSourcePath(SourcePath.FilePath);
		}

		if (IS_PROPERTY(bOptimizedForDynamicData))
		{
			SetOptimizedForDynamicData(bOptimizedForDynamicData);
		}

		if (IS_PROPERTY(MaxCollisionError))
		{
			if (MaxCollisionError < Octree.GetEstimatedPointSpacing())
			{
				FMessageDialog::Open(EAppMsgType::Type::Ok, FText::FromString(FString::Printf(TEXT("Average point spacing is estimated to be around %f cm.\nSetting accuracy close to or lower than that value may result in collision holes."), FMath::RoundToFloat(Octree.GetEstimatedPointSpacing() * 100) * 0.01f)));
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

int32 ULidarPointCloud::GetDataSize() const
{
	const int64 OctreeSize = Octree.GetAllocatedSize();
	const int64 CollisionSize = Octree.GetCollisionData()->Indices.GetAllocatedSize() + Octree.GetCollisionData()->Vertices.GetAllocatedSize();

	return (OctreeSize + CollisionSize) >> 20;
}

void ULidarPointCloud::RefreshBounds()
{
	Octree.RefreshBounds();
}

bool ULidarPointCloud::HasCollisionData() const
{
	return Octree.HasCollisionData();
}

int32 ULidarPointCloud::GetColliderPolys() const
{
	return Octree.HasCollisionData() ? Octree.GetCollisionData()->Indices.Num() : 0;
}

void ULidarPointCloud::RefreshRendering()
{
	Octree.MarkRenderDataDirty();
	OnPointCloudRebuiltEvent.Broadcast();
}

void ULidarPointCloud::GetPoints(TArray64<FLidarPointCloudPoint*>& Points, int64 StartIndex /*= 0*/, int64 Count /*= -1*/)
{
	GetPoints_Internal(Points, StartIndex, Count);
}

void ULidarPointCloud::GetPoints(TArray<FLidarPointCloudPoint*>& Points, int64 StartIndex /*= 0*/, int64 Count /*= -1*/)
{
	GetPoints_Internal(Points, StartIndex, Count);
}

void ULidarPointCloud::GetPointsInSphere(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
{
	GetPointsInSphere_Internal(SelectedPoints, Sphere, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInSphere(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
{
	GetPointsInSphere_Internal(SelectedPoints, Sphere, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInBox(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
{
	GetPointsInBox_Internal(SelectedPoints, Box, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInBox(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
{
	GetPointsInBox_Internal(SelectedPoints, Box, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInConvexVolume(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, ConvexVolume, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInConvexVolume(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, ConvexVolume, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInFrustum(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, Frustum, bVisibleOnly);
}

void ULidarPointCloud::GetPointsInFrustum(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, Frustum, bVisibleOnly);
}

TArray<FLidarPointCloudPoint> ULidarPointCloud::GetPointsAsCopies(bool bReturnWorldSpace, int32 StartIndex, int32 Count) const
{
	TArray<FLidarPointCloudPoint> Points;
	GetPointsAsCopies(Points, bReturnWorldSpace, StartIndex, Count);
	return Points;
}

void ULidarPointCloud::GetPointsAsCopies(TArray64<FLidarPointCloudPoint>& Points, bool bReturnWorldSpace, int64 StartIndex /*= 0*/, int64 Count /*= -1*/) const
{
	GetPointsAsCopies_Internal(Points, bReturnWorldSpace, StartIndex, Count);
}

void ULidarPointCloud::GetPointsAsCopies(TArray<FLidarPointCloudPoint>& Points, bool bReturnWorldSpace, int64 StartIndex /*= 0*/, int64 Count /*= -1*/) const
{
	GetPointsAsCopies_Internal(Points, bReturnWorldSpace, StartIndex, Count);
}

TArray<FLidarPointCloudPoint> ULidarPointCloud::GetPointsInSphereAsCopies(FVector Center, float Radius, bool bVisibleOnly, bool bReturnWorldSpace)
{
	TArray<FLidarPointCloudPoint> Points;
	GetPointsInSphereAsCopies(Points, FSphere(Center, Radius), bVisibleOnly, bReturnWorldSpace);
	return Points;
}

void ULidarPointCloud::GetPointsInSphereAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, bool bReturnWorldSpace) const
{
	GetPointsInSphereAsCopies_Internal(SelectedPoints, Sphere, bVisibleOnly, bReturnWorldSpace);
}

void ULidarPointCloud::GetPointsInSphereAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, bool bReturnWorldSpace) const
{
	GetPointsInSphereAsCopies_Internal(SelectedPoints, Sphere, bVisibleOnly, bReturnWorldSpace);
}

TArray<FLidarPointCloudPoint> ULidarPointCloud::GetPointsInBoxAsCopies(FVector Center, FVector Extent, bool bVisibleOnly, bool bReturnWorldSpace)
{
	TArray<FLidarPointCloudPoint> Points;
	GetPointsInBoxAsCopies(Points, FBox(Center - Extent, Center + Extent), bVisibleOnly, bReturnWorldSpace);
	return Points;
}

void ULidarPointCloud::GetPointsInBoxAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, bool bReturnWorldSpace) const
{
	GetPointsInBoxAsCopies_Internal(SelectedPoints, Box, bVisibleOnly, bReturnWorldSpace);
}

void ULidarPointCloud::GetPointsInBoxAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, bool bReturnWorldSpace) const
{
	GetPointsInBoxAsCopies_Internal(SelectedPoints, Box, bVisibleOnly, bReturnWorldSpace);
}

bool ULidarPointCloud::LineTraceSingle(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, FLidarPointCloudPoint& PointHit)
{
	FLidarPointCloudPoint* Point = LineTraceSingle(FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly);
	if (Point)
	{
		PointHit = *Point;
		return true;
	}

	return false;
}

#if WITH_EDITOR
void ULidarPointCloud::SelectByConvexVolume(FConvexVolume ConvexVolume, bool bAdditive, bool bApplyLocationOffset, bool bVisibleOnly)
{
	if(bApplyLocationOffset)
	{
		for(FPlane& Plane : ConvexVolume.Planes)
		{
			Plane = Plane.TranslateBy(-LocationOffset);
		}

		ConvexVolume.Init();
	}

	Octree.SelectByConvexVolume(ConvexVolume, bAdditive, bVisibleOnly);
}

void ULidarPointCloud::SelectBySphere(FSphere Sphere, bool bAdditive, bool bApplyLocationOffset, bool bVisibleOnly)
{
	if(bApplyLocationOffset)
	{
		Sphere.Center -= LocationOffset;
	}

	Octree.SelectBySphere(Sphere, bAdditive, bVisibleOnly);
}

void ULidarPointCloud::HideSelected()
{
	Octree.HideSelected();
}

void ULidarPointCloud::DeleteSelected()
{
	Octree.DeleteSelected();
}

void ULidarPointCloud::InvertSelection()
{
	Octree.InvertSelection();
}

int64 ULidarPointCloud::NumSelectedPoints() const
{
	return Octree.NumSelectedPoints();
}

bool ULidarPointCloud::HasSelectedPoints() const
{
	return Octree.HasSelectedPoints();
}

template <typename T>
void ULidarPointCloud::GetSelectedPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints,	FTransform Transform) const
{
	Transform.AddToTranslation(LocationOffset);
	Octree.GetSelectedPointsAsCopies(SelectedPoints, Transform);
}

void ULidarPointCloud::GetSelectedPointsAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const
{
	GetSelectedPointsAsCopies_Internal(SelectedPoints, Transform);
}

void ULidarPointCloud::GetSelectedPointsAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const
{
	GetSelectedPointsAsCopies_Internal(SelectedPoints, Transform);
}

void ULidarPointCloud::CalculateNormalsForSelection()
{
	TArray64<FLidarPointCloudPoint*>* SelectedPoints = new TArray64<FLidarPointCloudPoint*>();
	Octree.GetSelectedPoints(*SelectedPoints);
	CalculateNormals(SelectedPoints, [SelectedPoints]
	{
		delete SelectedPoints;
	});
}

void ULidarPointCloud::ClearSelection()
{
	Octree.ClearSelection();
}

void ULidarPointCloud::BuildStaticMeshBuffersForSelection(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform)
{
	Octree.BuildStaticMeshBuffersForSelection(CellSize, OutMeshBuffers, Transform);
}
#endif

void ULidarPointCloud::SetSourcePath(const FString& NewSourcePath)
{
	SourcePath.FilePath = NewSourcePath;

	if (FPaths::FileExists(SourcePath.FilePath))
	{
		if (FPaths::IsRelative(SourcePath.FilePath))
		{
			SourcePath.FilePath = FPaths::ConvertRelativePathToFull(SourcePath.FilePath);
		}

		// Generate new ImportSettings if the source path has changed
		ImportSettings = ULidarPointCloudFileIO::GetImportSettings(SourcePath.FilePath);
	}
	else
	{
		// Invalidate ImportSettings if the source path is invalid too
		ImportSettings = nullptr;
	}
}

void ULidarPointCloud::SetOptimizedForDynamicData(bool bNewOptimizedForDynamicData)
{
#if WITH_EDITOR
	FScopedSlowTask ProgressDialog(1, LOCTEXT("Optimize", "Optimizing Point Clouds..."));
	ProgressDialog.MakeDialog();
#endif

	bOptimizedForDynamicData = bNewOptimizedForDynamicData;

	if (bOptimizedForDynamicData)
	{
		Octree.OptimizeForDynamicData();
	}
	else
	{
		Octree.OptimizeForStaticData();
	}
}

void ULidarPointCloud::SetOptimalCollisionError()
{
	MaxCollisionError = FMath::CeilToInt(Octree.GetEstimatedPointSpacing() * 300) * 0.01f;
}

void ULidarPointCloud::BuildCollisionWithCallback(UObject* WorldContextObject, FLatentActionInfo LatentInfo, bool& bSuccess)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FPointCloudLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FPointCloudSimpleLatentAction* CompletionAction = new FPointCloudSimpleLatentAction(LatentInfo);

			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, CompletionAction);

			BuildCollision([CompletionAction, &bSuccess](bool bCompletedSuccessfully)
			{
				bSuccess = bCompletedSuccessfully;
				CompletionAction->bComplete = true;
			});
		}
	}
}

void ULidarPointCloud::BuildCollision(TFunction<void(bool)> CompletionCallback)
{
	if (bCollisionBuildInProgress)
	{
		PC_ERROR("Another collision operation already in progress.");
		if(CompletionCallback)
		{
			CompletionCallback(false);
		}
		return;
	}

	TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification = Notifications.Create("Building Collision", nullptr, "LidarPointCloudEditor.BuildCollision");
	
	bCollisionBuildInProgress = true;
	MarkPackageDirty();

	NewBodySetup = NewObject<UBodySetup>(this);
	NewBodySetup->BodySetupGuid = FGuid::NewGuid();
	NewBodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;
	NewBodySetup->bHasCookedCollisionData = true;

	TWeakObjectPtr<ULidarPointCloud> WeakThis = this;

	Async(EAsyncExecution::Thread, [WeakThis = MoveTemp(WeakThis), Notification, CompletionCallback = MoveTemp(CompletionCallback)]() mutable
	{
		if (ULidarPointCloud* RawPtr = WeakThis.Get())
		{
			FGCObjectScopeGuard Guard(RawPtr);
			RawPtr->Octree.BuildCollision(RawPtr->MaxCollisionError, true);

			FBenchmarkTimer::Reset();

			RawPtr->NewBodySetup->CreatePhysicsMeshes();
			AsyncTask(ENamedThreads::GameThread, [WeakThis = MoveTemp(WeakThis), Notification, CompletionCallback = MoveTemp(CompletionCallback)]
			{
				if (ULidarPointCloud* RawPtr = WeakThis.Get())
				{
					RawPtr->FinishPhysicsAsyncCook(true, Notification);
					if(CompletionCallback)
					{
						CompletionCallback(true);
					}
				}
			});
		}
	});
}

void ULidarPointCloud::RemoveCollision()
{
	if (bCollisionBuildInProgress)
	{
		PC_ERROR("Another collision operation already in progress.");
		return;
	}

	bCollisionBuildInProgress = true;
	
	MarkPackageDirty();

	Octree.RemoveCollision();

	BodySetup = NewObject<UBodySetup>(this);
	ReleaseCollisionRendering(false);
	InitializeCollisionRendering();
	OnPointCloudUpdateCollisionEvent.Broadcast();

	bCollisionBuildInProgress = false;
}

void ULidarPointCloud::BuildStaticMeshBuffers(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform)
{
	Octree.BuildStaticMeshBuffers(CellSize, OutMeshBuffers, Transform);
}

void ULidarPointCloud::SetLocationOffset(FVector Offset)
{
	LocationOffset = Offset;
	MarkPackageDirty();
	OnPointCloudRebuiltEvent.Broadcast();
}

void ULidarPointCloud::Reimport(const FLidarPointCloudAsyncParameters& AsyncParameters)
{
	if (FPaths::FileExists(SourcePath.FilePath))
	{
		FScopeTryLock Lock(&ProcessingLock);

		if (!Lock.IsLocked())
		{
			PC_ERROR("Cannot reimport the asset - data is currently being used.");
			return;
		}

		bAsyncCancelled = false;
		TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification = Notifications.Create("Importing Point Cloud", &bAsyncCancelled);

		const bool bCenter = GetDefault<ULidarPointCloudSettings>()->bAutoCenterOnImport;

		// Reset the data optimization flag on import
		bOptimizedForDynamicData = false;

		// The actual import function to be executed
		auto ImportFunction = [this, Notification, AsyncParameters, bCenter]
		{
			// This will take over the lock
			FScopeLock Lock(&ProcessingLock);

			bool bSuccess = false;

			// Wait for rendering to complete before proceeding and lock the access to the data
			FScopeLock DataLock(&Octree.DataLock);

			FLidarPointCloudImportResults ImportResults;

			// If the file supports concurrent insertion, we can stream the data in chunks and perform async insertion at the same time
			if (ULidarPointCloudFileIO::FileSupportsConcurrentInsertion(SourcePath.FilePath))
			{
				PC_LOG("Using Concurrent Insertion");

				ImportResults = FLidarPointCloudImportResults(&bAsyncCancelled,
				[this, Notification, AsyncParameters](float Progress)
				{
					Notification->SetProgress(100.0f * Progress);
					if (AsyncParameters.ProgressCallback)
					{
						AsyncParameters.ProgressCallback(100.0f * Progress);
					}
				},
				[this](const FBox& Bounds, FVector InOriginalCoordinates)
				{
					Initialize(Bounds.ShiftBy(-InOriginalCoordinates));
				},
				[this](TArray64<FLidarPointCloudPoint>* Points)
				{
					Octree.InsertPoints(Points->GetData(), Points->Num(), GetDefault<ULidarPointCloudSettings>()->DuplicateHandling, false, (FVector3f)-LocationOffset);
				});

				bSuccess = ULidarPointCloudFileIO::Import(SourcePath.FilePath, ImportSettings, ImportResults);
			}
			else
			{
				ImportResults = FLidarPointCloudImportResults(&bAsyncCancelled, [this, Notification, AsyncParameters](float Progress)
				{
					Notification->SetProgress(50.0f * Progress);
					if (AsyncParameters.ProgressCallback)
					{
						AsyncParameters.ProgressCallback(50.0f * Progress);
					}
				});

				if (ULidarPointCloudFileIO::Import(SourcePath.FilePath, ImportSettings, ImportResults))
				{
					// Re-initialize the Octree
					Initialize(ImportResults.Bounds);

					FScopeBenchmarkTimer BenchmarkTimer("Octree Build-Up");

					bSuccess = InsertPoints_NoLock(ImportResults.Points.GetData(), ImportResults.Points.Num(), GetDefault<ULidarPointCloudSettings>()->DuplicateHandling, false, -LocationOffset, &bAsyncCancelled, [this, Notification, AsyncParameters](float Progress)
					{
						Notification->SetProgress(50.0f + 50.0f * Progress);
						if (AsyncParameters.ProgressCallback)
						{
							AsyncParameters.ProgressCallback(50.0f + 50.0f * Progress);
						}
					});

					if (!bSuccess)
					{
						BenchmarkTimer.bActive = false;
					}
				}
			}

			if (bSuccess)
			{
				ClassificationsImported = ImportResults.ClassificationsImported;

				RefreshBounds();
				OriginalCoordinates = LocationOffset + ImportResults.OriginalCoordinates;

				// Show the cloud at its original location, if selected
				LocationOffset = bCenter ? FVector::ZeroVector : OriginalCoordinates;

				// Adjust default max collision error
				SetOptimalCollisionError();
			}
			else
			{
				Octree.Empty(true);

				OriginalCoordinates = FVector::ZeroVector;
				LocationOffset = FVector::ZeroVector;

				// Update PointCloudAssetRegistryCache
				PointCloudAssetRegistryCache.PointCount = FString::FromInt(Octree.GetNumPoints());
			}

			// Only process those if not being destroyed
			if (!HasAnyFlags(RF_BeginDestroyed))
			{
				auto PostFunction = [this, Notification, bSuccess]() {
					MarkPackageDirty();
					Notification->Close(bSuccess);
					OnPointCloudRebuiltEvent.Broadcast();

					if (bSuccess)
					{
						FScopeLock Lock(&ProcessingLock);

						if (GetDefault<ULidarPointCloudSettings>()->bAutoCalculateNormalsOnImport)
						{
							CalculateNormals(nullptr, nullptr);
						}
						
						if (GetDefault<ULidarPointCloudSettings>()->bAutoBuildCollisionOnImport)
						{
							BuildCollision();
						}

					}
				};

				// Make sure the call is executed on the correct thread if using async
				if (IsInGameThread())
				{
					PostFunction();
				}
				else
				{
					AsyncTask(ENamedThreads::GameThread, MoveTemp(PostFunction));
				}
			}	
			
			if (AsyncParameters.CompletionCallback)
			{
				AsyncParameters.CompletionCallback(bSuccess);
			}

			if(!bSuccess)
			{
				PC_ERROR("Point Cloud importing failed or cancelled.");
			}
		};

		if (AsyncParameters.bUseAsync)
		{
			Async(EAsyncExecution::Thread, MoveTemp(ImportFunction));
		}
		else
		{
			ImportFunction();
		}
	}
	else
	{
		PC_ERROR("Reimport failed, provided source path '%s' could not be found.", *SourcePath.FilePath);

		if (AsyncParameters.CompletionCallback)
		{
			AsyncParameters.CompletionCallback(false);
		}
	}
}

void ULidarPointCloud::Reimport(UObject* WorldContextObject, bool bUseAsync, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FPointCloudLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			AsyncMode = ELidarPointCloudAsyncMode::Progress;
			FPointCloudLatentAction* CompletionAction = new FPointCloudLatentAction(LatentInfo, AsyncMode);

			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, CompletionAction);

			Reimport(FLidarPointCloudAsyncParameters(bUseAsync,
				[&Progress, &AsyncMode](float InProgress)
				{
					Progress = InProgress;
				},
				[&AsyncMode](bool bSuccess)
				{
					AsyncMode = bSuccess ? ELidarPointCloudAsyncMode::Success : ELidarPointCloudAsyncMode::Failure;
				}));
		}
	}
}

bool ULidarPointCloud::Export(const FString& Filename)
{
	return ULidarPointCloudFileIO::Export(Filename, this);
}

void ULidarPointCloud::InsertPoint(const FLidarPointCloudPoint& Point, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation)
{
	FScopeLock Lock(&Octree.DataLock);

	Octree.InsertPoint(&Point, DuplicateHandling, bRefreshPointsBounds, (FVector3f)Translation);

	// Update PointCloudAssetRegistryCache
	PointCloudAssetRegistryCache.PointCount = FString::FromInt(Octree.GetNumPoints());
}

bool ULidarPointCloud::InsertPoints(FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return InsertPoints_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

bool ULidarPointCloud::InsertPoints(const FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return InsertPoints_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

bool ULidarPointCloud::InsertPoints(FLidarPointCloudPoint** InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return InsertPoints_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

bool ULidarPointCloud::InsertPoints_NoLock(FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return InsertPoints_NoLock_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

bool ULidarPointCloud::InsertPoints_NoLock(const FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return InsertPoints_NoLock_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

bool ULidarPointCloud::InsertPoints_NoLock(FLidarPointCloudPoint** InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return InsertPoints_NoLock_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

template<typename T>
bool ULidarPointCloud::InsertPoints_NoLock_Internal(T InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled, TFunction<void(float)> ProgressCallback)
{
	if (bOptimizedForDynamicData)
	{
		Octree.InsertPoints(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, (FVector3f)Translation);
		
		if (ProgressCallback)
		{
			ProgressCallback(1.0);
		}
		
		return true;
	}
	else
	{
		const int32 MaxBatchSize = GetDefault<ULidarPointCloudSettings>()->MultithreadingInsertionBatchSize;

		// Minimum amount of points to progress to count as 1%
		int64 RefreshStatusFrequency = Count * 0.01f;
		FThreadSafeCounter64 ProcessedPoints(0);
		int64 TotalProcessedPoints = 0;

		const int32 NumThreads = FMath::Min(FPlatformMisc::NumberOfCoresIncludingHyperthreads() - 1, (int32)(Count / MaxBatchSize) + 1);
		TArray<TFuture<void>>ThreadResults;
		ThreadResults.Reserve(NumThreads);
		const int64 NumPointsPerThread = Count / NumThreads + 1;

		FCriticalSection ProgressCallbackLock;

		// Fire threads
		for (int32 ThreadID = 0; ThreadID < NumThreads; ThreadID++)
		{
			ThreadResults.Add(Async(EAsyncExecution::Thread, [this, ThreadID, DuplicateHandling, bRefreshPointsBounds, MaxBatchSize, NumPointsPerThread, RefreshStatusFrequency, &ProcessedPoints, &TotalProcessedPoints, InPoints, Count, &ProgressCallback, &ProgressCallbackLock, &bCanceled, &Translation]
				{
					int64 Idx = ThreadID * NumPointsPerThread;
					int64 MaxIdx = FMath::Min(Idx + NumPointsPerThread, Count);
					T DataPointer = InPoints + Idx;

					while (Idx < MaxIdx)
					{
						int32 BatchSize = FMath::Min(MaxIdx - Idx, (int64)MaxBatchSize);

						Octree.InsertPoints(DataPointer, BatchSize, DuplicateHandling, bRefreshPointsBounds, (FVector3f)Translation);

						if (ProgressCallback)
						{
							ProcessedPoints.Add(BatchSize);
							if (ProcessedPoints.GetValue() > RefreshStatusFrequency)
							{
								FScopeLock Lock(&ProgressCallbackLock);
								TotalProcessedPoints += ProcessedPoints.GetValue();
								ProcessedPoints.Reset();
								ProgressCallback((double)TotalProcessedPoints / Count);
							}
						}

						if (bCanceled && *bCanceled)
						{
							break;
						}

						Idx += BatchSize;
						DataPointer += BatchSize;
					}
				}));
		}

		// Sync
		for (const TFuture<void>& ThreadResult : ThreadResults)
		{
			ThreadResult.Get();
		}

		// Do not attempt to touch Render Data if being destroyed
		if (!HasAnyFlags(RF_BeginDestroyed))
		{
			// Update PointCloudAssetRegistryCache
			PointCloudAssetRegistryCache.PointCount = FString::FromInt(Octree.GetNumPoints());
		}

		return !bCanceled || !(*bCanceled);
	}
}

template<typename T>
bool ULidarPointCloud::SetData_Internal(T Points, const int64& Count, TFunction<void(float)> ProgressCallback)
{
	// Lock the point cloud
	FScopeLock Lock(&ProcessingLock);

	// Calculate the bounds
	FBox Bounds = CalculateBoundsFromPoints(Points, Count);

	bool bSuccess = false;

	// Only proceed if the bounds are valid
	if (Bounds.IsValid)
	{
		// Wait for rendering to complete before proceeding and lock the access to the data
		FScopeLock DataLock(&Octree.DataLock);

		// Initialize the Octree
		Initialize(Bounds);

		bSuccess = InsertPoints_NoLock(Points, Count, GetDefault<ULidarPointCloudSettings>()->DuplicateHandling, false, -LocationOffset, nullptr, MoveTemp(ProgressCallback));

		if (!bSuccess)
		{
			Octree.Empty(true);
		}

		// Only process those if not being destroyed
		if (!HasAnyFlags(RF_BeginDestroyed))
		{
			auto PostFunction = [this, bSuccess]() {
				MarkPackageDirty();
				OnPointCloudRebuiltEvent.Broadcast();
			};

			// Make sure the call is executed on the correct thread if using async
			if (IsInGameThread())
			{
				PostFunction();
			}
			else
			{
				AsyncTask(ENamedThreads::GameThread, MoveTemp(PostFunction));
			}
		}
	}

	if (!bSuccess)
	{
		PC_ERROR("Setting Point Cloud data failed.");
	}

	return bSuccess;
}

bool ULidarPointCloud::SetData(const FLidarPointCloudPoint* Points, const int64& Count, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return SetData_Internal(Points, Count, MoveTemp(ProgressCallback));
}

bool ULidarPointCloud::SetData(FLidarPointCloudPoint** Points, const int64& Count, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	return SetData_Internal(Points, Count, MoveTemp(ProgressCallback));
}

void ULidarPointCloud::Merge(TArray<ULidarPointCloud*> PointCloudsToMerge, TFunction<void(void)> ProgressCallback)
{
	for (int32 i = 0; i < PointCloudsToMerge.Num(); i++)
	{
		if (!IsValid(PointCloudsToMerge[i]) || PointCloudsToMerge[i] == this)
		{
			PointCloudsToMerge.RemoveAtSwap(i--, EAllowShrinking::No);
		}
	}

	PointCloudsToMerge.Shrink();

	// Abort if no valid assets are found
	if (PointCloudsToMerge.Num() == 0)
	{
		return;
	}

	FScopeBenchmarkTimer Timer("Merge");

	// Lock the point cloud
	FScopeLock Lock(&ProcessingLock);
	FScopeLock DataLock(&Octree.DataLock);

	if (ProgressCallback)
	{
		ProgressCallback();
	}

	// Calculate new, combined bounds
	FBox NewBounds(EForceInit::ForceInit);
	FBox NewAbsoluteBounds(EForceInit::ForceInit);

	// Only include this asset if it actually has any data
	if (GetNumPoints() > 0)
	{
		NewBounds += GetBounds(false);
		NewAbsoluteBounds += GetBounds(true);
	}

	for (ULidarPointCloud* Asset : PointCloudsToMerge)
	{
		NewBounds += Asset->GetBounds(false);
		NewAbsoluteBounds += Asset->GetBounds(true);

		for (uint8& Classification : Asset->ClassificationsImported)
		{
			ClassificationsImported.AddUnique(Classification);
		}
	}

	// Make a copy of current points, as the data will be reinitialized
	TArray<FLidarPointCloudPoint> Points;
	GetPointsAsCopies(Points, false);

	FVector OldLocationOffset = LocationOffset;

	// Initialize the Octree
	Initialize(NewBounds);

	OriginalCoordinates = NewAbsoluteBounds.GetCenter();

	// Re-insert original points
	InsertPoints(Points, GetDefault<ULidarPointCloudSettings>()->DuplicateHandling, false, OldLocationOffset - LocationOffset);

	Points.Empty();

	TArray<TFuture<void>> ThreadResults;

	const ULidarPointCloudSettings* Settings = GetDefault<ULidarPointCloudSettings>();
	const int32 MaxBatchSize = Settings->MultithreadingInsertionBatchSize;
	const ELidarPointCloudDuplicateHandling DuplicateHandling = Settings->DuplicateHandling;

	// Insert other points
	for (ULidarPointCloud* Asset : PointCloudsToMerge)
	{
		if (ProgressCallback)
		{
			ProgressCallback();
		}

		const FVector3f Translation = (FVector3f)(Asset->LocationOffset - LocationOffset);
		Asset->Octree.GetPointsAsCopiesInBatches([this, &ThreadResults, DuplicateHandling, Translation](TSharedPtr<TArray64<FLidarPointCloudPoint>> Points)
		{
			ThreadResults.Add(Async(EAsyncExecution::ThreadPool, [this, Points, DuplicateHandling, Translation]() {
				Octree.InsertPoints(Points->GetData(), Points->Num(), DuplicateHandling, false, Translation);
			}));
		}, MaxBatchSize, false);
	}

	// Sync
	if (ProgressCallback)
	{
		ProgressCallback();
	}

	for (const TFuture<void>& ThreadResult : ThreadResults)
	{
		ThreadResult.Get();
	}

	SetOptimalCollisionError();
	
	MarkPackageDirty();
	OnPointCloudRebuiltEvent.Broadcast();
}

void ULidarPointCloud::CalculateNormals(FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GetWorld())
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FPointCloudLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			ELidarPointCloudAsyncMode AsyncMode = ELidarPointCloudAsyncMode::Progress;
			FPointCloudLatentAction* CompletionAction = new FPointCloudLatentAction(LatentInfo, AsyncMode);
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, CompletionAction);
			CalculateNormals(nullptr, [&AsyncMode] { AsyncMode = ELidarPointCloudAsyncMode::Success; });
		}
	}
}

void ULidarPointCloud::CalculateNormals(TArray64<FLidarPointCloudPoint*>* Points, TFunction<void(void)> CompletionCallback)
{
	FScopeTryLock Lock(&ProcessingLock);

	if (!Lock.IsLocked())
	{
		PC_ERROR("Cannot calculate normals for the asset - data is currently being used.");
		return;
	}

	bAsyncCancelled = false;
	TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification = Notifications.Create("Calculating Normals", &bAsyncCancelled);
	Async(EAsyncExecution::Thread,
	[this, Points]
	{
		// This will take over the lock
		FScopeLock Lock(&ProcessingLock);

		// Wait for rendering to complete before proceeding and lock the access to the data
		FScopeLock DataLock(&Octree.DataLock);

		Octree.CalculateNormals(&bAsyncCancelled, NormalsQuality, NormalsNoiseTolerance, Points);
	},
	[this, Notification, _CompletionCallback = MoveTemp(CompletionCallback)]
	{
		AsyncTask(ENamedThreads::GameThread, [this, Notification]
		{
			MarkPackageDirty();
			Notification->Close(!bAsyncCancelled);
			OnPointCloudNormalsUpdatedEvent.Broadcast();
		});

		if (_CompletionCallback)
		{
			_CompletionCallback();
		}
	});
}

bool ULidarPointCloud::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
	OutTriMeshEstimates.VerticeCount = Octree.GetCollisionData()->Vertices.Num();
	
	return true;
}

bool ULidarPointCloud::GetPhysicsTriMeshData(FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	CollisionData->Vertices = Octree.GetCollisionData()->Vertices;
	CollisionData->Indices = Octree.GetCollisionData()->Indices;

	return true;
}

UBodySetup* ULidarPointCloud::GetBodySetup()
{
	return IsValid(BodySetup) ? BodySetup : nullptr;
}

void ULidarPointCloud::AlignClouds(TArray<ULidarPointCloud*> PointCloudsToAlign)
{
	FBox CombinedBounds(EForceInit::ForceInit);

	// Calculate combined bounds
	for (ULidarPointCloud* Asset : PointCloudsToAlign)
	{
		CombinedBounds += Asset->GetBounds(true);
	}

	// Calculate and apply individual shifts
	for (ULidarPointCloud* Asset : PointCloudsToAlign)
	{
		Asset->SetLocationOffset(Asset->OriginalCoordinates - CombinedBounds.GetCenter());
	}
}

ULidarPointCloud* ULidarPointCloud::CreateFromFile(const FString& Filename, const FLidarPointCloudAsyncParameters& AsyncParameters, TSharedPtr<struct FLidarPointCloudImportSettings> ImportSettings, UObject* InParent, FName InName, EObjectFlags Flags)
{
#if WITH_EDITOR
	FOnPointCloudChanged OnPointCloudRebuiltEvent;
	FOnPointCloudChanged OnPointCloudUpdateCollisionEvent;
	bool bOldPointCloudExists = false;

	// See if Point Cloud already exists
	ULidarPointCloud* OldPointCloud = Cast<ULidarPointCloud>(StaticFindObjectFast(nullptr, InParent, InName, EFindObjectFlags::ExactClass));
	if (OldPointCloud)
	{
		bOldPointCloudExists = true;

		// If so, store event references to re-apply to the new object
		OnPointCloudRebuiltEvent = OldPointCloud->OnPointCloudRebuiltEvent;
		OnPointCloudUpdateCollisionEvent = OldPointCloud->OnPointCloudUpdateCollisionEvent;
	}
#endif

	ULidarPointCloud* PointCloud = NewObject<ULidarPointCloud>(InParent, InName, Flags);

#if WITH_EDITOR
	if (bOldPointCloudExists)
	{
		PointCloud->OnPointCloudRebuiltEvent = OnPointCloudRebuiltEvent;
		PointCloud->OnPointCloudUpdateCollisionEvent = OnPointCloudUpdateCollisionEvent;
	}
#endif

	PointCloud->SetSourcePath(Filename);
	PointCloud->ImportSettings = ImportSettings;
	PointCloud->Reimport(AsyncParameters);

	return PointCloud;
}

template<typename T>
ULidarPointCloud* ULidarPointCloud::CreateFromData(T Points, const int64& Count, const FLidarPointCloudAsyncParameters& AsyncParameters)
{
	ULidarPointCloud* PC = NewObject<ULidarPointCloud>();

	// Process points, if there are any available
	if (Points && Count > 0)
	{
		// Start the process
		if (AsyncParameters.bUseAsync)
		{
			Async(EAsyncExecution::Thread, [PC, AsyncParameters, Points, Count]
			{
				bool bSuccess = PC->SetData(Points, Count, AsyncParameters.ProgressCallback);				
				if (AsyncParameters.CompletionCallback)
				{
					AsyncParameters.CompletionCallback(bSuccess);
				}
			});
		}
		else
		{
			PC->SetData(Points, Count);
		}
	}

	return PC;
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(const TArray<FLidarPointCloudPoint>& Points, const bool& bUseAsync)
{
	return CreateFromData(Points.GetData(), Points.Num(), FLidarPointCloudAsyncParameters(bUseAsync));
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(const TArray64<FLidarPointCloudPoint>& Points, const bool& bUseAsync)
{
	return CreateFromData(Points.GetData(), Points.Num(), FLidarPointCloudAsyncParameters(bUseAsync));
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(const TArray<FLidarPointCloudPoint>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters)
{
	return CreateFromData(Points.GetData(), Points.Num(), AsyncParameters);
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(const TArray64<FLidarPointCloudPoint>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters)
{
	return CreateFromData(Points.GetData(), Points.Num(), AsyncParameters);
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(TArray<FLidarPointCloudPoint*>& Points, const bool& bUseAsync)
{
	return CreateFromData(Points.GetData(), Points.Num(), FLidarPointCloudAsyncParameters(bUseAsync));
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(TArray64<FLidarPointCloudPoint*>& Points, const bool& bUseAsync)
{
	return CreateFromData(Points.GetData(), Points.Num(), FLidarPointCloudAsyncParameters(bUseAsync));
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(TArray<FLidarPointCloudPoint*>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters)
{
	return CreateFromData(Points.GetData(), Points.Num(), AsyncParameters);
}

ULidarPointCloud* ULidarPointCloud::CreateFromData(TArray64<FLidarPointCloudPoint*>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters)
{
	return CreateFromData(Points.GetData(), Points.Num(), AsyncParameters);
}

FBox ULidarPointCloud::CalculateBoundsFromPoints(const FLidarPointCloudPoint* Points, const int64& Count)
{
	FBox Bounds(EForceInit::ForceInit);

	// Process points, if there are any available
	if (Points && Count > 0)
	{
		for (const FLidarPointCloudPoint* Data = Points, *DataEnd = Data + Count; Data != DataEnd; ++Data)
		{
			Bounds += (FVector)Data->Location;
		}
	}

	return Bounds;
}

FBox ULidarPointCloud::CalculateBoundsFromPoints(FLidarPointCloudPoint** Points, const int64& Count)
{
	FBox Bounds(EForceInit::ForceInit);

	// Process points, if there are any available
	if (Points && Count > 0)
	{
		for (FLidarPointCloudPoint** Data = Points, **DataEnd = Data + Count; Data != DataEnd; ++Data)
		{
			Bounds += (FVector)(*Data)->Location;
		}
	}

	return Bounds;
}

void ULidarPointCloud::FinishPhysicsAsyncCook(bool bSuccess, TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification)
{
	FBenchmarkTimer::Log("CookingCollision");
	Notification->Close(bSuccess);

	if (bSuccess)
	{
		BodySetup = NewBodySetup;
		ReleaseCollisionRendering(false); 
		InitializeCollisionRendering();
		OnPointCloudUpdateCollisionEvent.Broadcast();
	}

	bCollisionBuildInProgress = false;
}

template <typename T>
void ULidarPointCloud::GetPoints_Internal(TArray<FLidarPointCloudPoint*, T>& Points, int64 StartIndex /*= 0*/, int64 Count /*= -1*/)
{
	Octree.GetPoints(Points, StartIndex, Count);
}

template <typename T>
void ULidarPointCloud::GetPointsInSphere_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, FSphere Sphere, const bool& bVisibleOnly)
{
	Sphere.Center -= LocationOffset;
	Octree.GetPointsInSphere(SelectedPoints, Sphere, bVisibleOnly);
}

template <typename T>
void ULidarPointCloud::GetPointsInBox_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
{
	Octree.GetPointsInBox(SelectedPoints, Box.ShiftBy(-LocationOffset), bVisibleOnly);
}

template <typename T>
void ULidarPointCloud::GetPointsInConvexVolume_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly)
{
	Octree.GetPointsInConvexVolume(SelectedPoints, ConvexVolume, bVisibleOnly);
}

template <typename T>
void ULidarPointCloud::GetPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& Points, bool bReturnWorldSpace, int64 StartIndex /*= 0*/, int64 Count /*= -1*/) const
{
	FTransform LocalToWorld(LocationOffset);
	Octree.GetPointsAsCopies(Points, bReturnWorldSpace ? &LocalToWorld : nullptr, StartIndex, Count);
}

template <typename T>
void ULidarPointCloud::GetPointsInSphereAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, FSphere Sphere, const bool& bVisibleOnly, bool bReturnWorldSpace) const
{
	FTransform LocalToWorld(LocationOffset);
	Sphere.Center -= LocationOffset;
	Octree.GetPointsInSphereAsCopies(SelectedPoints, Sphere, bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr);
}

template <typename T>
void ULidarPointCloud::GetPointsInBoxAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, bool bReturnWorldSpace) const
{
	FTransform LocalToWorld(LocationOffset);
	Octree.GetPointsInBoxAsCopies(SelectedPoints, Box.ShiftBy(-LocationOffset), bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr);
}

template<typename T>
bool ULidarPointCloud::InsertPoints_Internal(T InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled /*= nullptr*/, TFunction<void(float)> ProgressCallback /*= TFunction<void(float)>()*/)
{
	FScopeLock Lock(&Octree.DataLock);
	return InsertPoints_NoLock_Internal(InPoints, Count, DuplicateHandling, bRefreshPointsBounds, Translation, bCanceled, MoveTemp(ProgressCallback));
}

/*********************************************************************************************** ULidarPointCloudBlueprintLibrary */

#define ITERATE_CLOUDS(Action)\
if (UWorld* World = WorldContextObject ? WorldContextObject->GetWorld() : nullptr)\
{\
	for (TActorIterator<ALidarPointCloudActor> Itr(World); Itr; ++Itr)\
	{\
		ALidarPointCloudActor* Actor = *Itr;\
		ULidarPointCloudComponent* Component = Actor->GetPointCloudComponent();\
		{ Action }\
	}\
}

void ULidarPointCloudBlueprintLibrary::CreatePointCloudFromFile(UObject* WorldContextObject, const FString& Filename, bool bUseAsync, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud)
{	
	CreatePointCloudFromFile(WorldContextObject, Filename, bUseAsync, LatentInfo, FLidarPointCloudImportSettings::MakeGeneric(Filename), AsyncMode, Progress, PointCloud);
}

void ULidarPointCloudBlueprintLibrary::CreatePointCloudFromFile(UObject* WorldContextObject, const FString& Filename, bool bUseAsync, FLatentActionInfo LatentInfo, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud)
{
	PointCloud = nullptr;
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FPointCloudLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			AsyncMode = ELidarPointCloudAsyncMode::Progress;
			FPointCloudLatentAction* CompletionAction = new FPointCloudLatentAction(LatentInfo, AsyncMode);

			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, CompletionAction);

			PointCloud = ULidarPointCloud::CreateFromFile(Filename, FLidarPointCloudAsyncParameters(bUseAsync,
				[&Progress, &AsyncMode](float InProgress)
				{
					Progress = InProgress;
				},
				[&AsyncMode](bool bSuccess)
				{
					AsyncMode = bSuccess ? ELidarPointCloudAsyncMode::Success : ELidarPointCloudAsyncMode::Failure;
				}),
				ImportSettings);
		}
	}
}

void ULidarPointCloudBlueprintLibrary::CreatePointCloudFromData(UObject* WorldContextObject, const TArray<FLidarPointCloudPoint>& Points, bool bUseAsync, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud)
{
	PointCloud = nullptr;
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FPointCloudLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			AsyncMode = ELidarPointCloudAsyncMode::Progress;
			FPointCloudLatentAction* CompletionAction = new FPointCloudLatentAction(LatentInfo, AsyncMode);

			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, CompletionAction);

			PointCloud = ULidarPointCloud::CreateFromData(Points, FLidarPointCloudAsyncParameters(bUseAsync,
				[&Progress, &AsyncMode](float InProgress)
				{
					Progress = InProgress;
				},
				[&AsyncMode](bool bSuccess)
				{
					AsyncMode = bSuccess ? ELidarPointCloudAsyncMode::Success : ELidarPointCloudAsyncMode::Failure;
				}));
		}
	}
}

bool ULidarPointCloudBlueprintLibrary::ArePointsInSphere(UObject* WorldContextObject, FVector Center, float Radius, bool bVisibleOnly)
{
	ITERATE_CLOUDS({
		if (Component->HasPointsInSphere(Center, Radius, bVisibleOnly))
		{
			return true;
		}
	});
	return false;
}

bool ULidarPointCloudBlueprintLibrary::ArePointsInBox(UObject* WorldContextObject, FVector Center, FVector Extent, bool bVisibleOnly)
{
	ITERATE_CLOUDS({
		if(Component->HasPointsInBox(Center, Extent, bVisibleOnly))
		{
			return true;
		} 
	});
	return false;
}

bool ULidarPointCloudBlueprintLibrary::ArePointsByRay(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly)
{
	ITERATE_CLOUDS({
		if(Component->HasPointsByRay(Origin, Direction, Radius, bVisibleOnly))
		{
			return true;
		} 
	});
	return false;
}

void ULidarPointCloudBlueprintLibrary::GetPointsInSphereAsCopies(UObject* WorldContextObject, TArray<FLidarPointCloudPoint>& SelectedPoints, FVector Center, float Radius, bool bVisibleOnly)
{
	SelectedPoints.Reset();

	const FSphere Sphere(Center, Radius);

	ITERATE_CLOUDS({
		TArray<FLidarPointCloudPoint> _SelectedPoints;
		Component->GetPointsInSphereAsCopies(_SelectedPoints, Sphere, bVisibleOnly, true);
		SelectedPoints.Append(_SelectedPoints);
	});
}

void ULidarPointCloudBlueprintLibrary::GetPointsInBoxAsCopies(UObject* WorldContextObject, TArray<FLidarPointCloudPoint>& SelectedPoints, FVector Center, FVector Extent, const bool& bVisibleOnly)
{
	SelectedPoints.Reset();

	const FBox Box(Center - Extent, Center + Extent);

	ITERATE_CLOUDS({
		TArray<FLidarPointCloudPoint> _SelectedPoints;
		Component->GetPointsInBoxAsCopies(_SelectedPoints, Box, bVisibleOnly, true);
		SelectedPoints.Append(_SelectedPoints);
	});
}

bool ULidarPointCloudBlueprintLibrary::LineTraceSingle(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, FLidarPointCloudTraceHit& Hit)
{
	const FLidarPointCloudRay Ray((FVector3f)Origin, (FVector3f)Direction);

	ITERATE_CLOUDS({
		if (FLidarPointCloudPoint* Point = Component->LineTraceSingle(Ray, Radius, bVisibleOnly))
		{
			Hit = FLidarPointCloudTraceHit(Actor, Component);
			Hit.Points.Add(*Point);
			return true;
		}
	});

	return false;
}

bool ULidarPointCloudBlueprintLibrary::LineTraceMulti(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, TArray<FLidarPointCloudTraceHit>& Hits)
{
	Hits.Reset();
	const FLidarPointCloudRay Ray((FVector3f)Origin, (FVector3f)Direction);

	ITERATE_CLOUDS({
		FLidarPointCloudTraceHit Hit(Actor, Component);
		if (Component->LineTraceMulti(Ray, Radius, bVisibleOnly, true, Hit.Points))
		{
			Hits.Add(Hit);
			return true;
		}
	});

	return Hits.Num() > 0;
}

void ULidarPointCloudBlueprintLibrary::SetVisibilityOfPointsInSphere(UObject* WorldContextObject, bool bNewVisibility, FVector Center, float Radius)
{
	ITERATE_CLOUDS({ Component->SetVisibilityOfPointsInSphere(bNewVisibility, Center, Radius); });
}

void ULidarPointCloudBlueprintLibrary::SetVisibilityOfPointsInBox(UObject* WorldContextObject, bool bNewVisibility, FVector Center, FVector Extent)
{
	ITERATE_CLOUDS({ Component->SetVisibilityOfPointsInBox(bNewVisibility, Center, Extent); });
}

void ULidarPointCloudBlueprintLibrary::SetVisibilityOfFirstPointByRay(UObject* WorldContextObject, bool bNewVisibility, FVector Origin, FVector Direction, float Radius)
{
	float MinDistance = FLT_MAX;
	ULidarPointCloudComponent* ClosestComponent = nullptr;

	const FLidarPointCloudRay Ray((FVector3f)Origin, (FVector3f)Direction);

	ITERATE_CLOUDS({
		if (FLidarPointCloudPoint* Point = Component->LineTraceSingle(Ray, Radius, false))
		{
			const float DistanceSq = ((FVector)Point->Location - Origin).SizeSquared();
			if (DistanceSq < MinDistance)
			{
				MinDistance = DistanceSq;
				ClosestComponent = Component;
			}
		}
	});

	if (ClosestComponent)
	{
		ClosestComponent->SetVisibilityOfFirstPointByRay(bNewVisibility, Ray, Radius);
	}
}

void ULidarPointCloudBlueprintLibrary::SetVisibilityOfPointsByRay(UObject* WorldContextObject, bool bNewVisibility, FVector Origin, FVector Direction, float Radius)
{
	ITERATE_CLOUDS({ Component->SetVisibilityOfPointsByRay(bNewVisibility, Origin, Direction, Radius); });
}

void ULidarPointCloudBlueprintLibrary::ApplyColorToPointsInSphere(UObject* WorldContextObject, FColor NewColor, FVector Center, float Radius, bool bVisibleOnly)
{
	ITERATE_CLOUDS({ Component->ApplyColorToPointsInSphere(NewColor, Center, Radius, bVisibleOnly); });
}

void ULidarPointCloudBlueprintLibrary::ApplyColorToPointsInBox(UObject* WorldContextObject, FColor NewColor, FVector Center, FVector Extent, bool bVisibleOnly)
{
	ITERATE_CLOUDS({ Component->ApplyColorToPointsInBox(NewColor, Center, Extent, bVisibleOnly); });
}

void ULidarPointCloudBlueprintLibrary::ApplyColorToFirstPointByRay(UObject* WorldContextObject, FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly)
{
	float MinDistance = FLT_MAX;
	ULidarPointCloudComponent* ClosestComponent = nullptr;

	const FLidarPointCloudRay Ray((FVector3f)Origin, (FVector3f)Direction);

	ITERATE_CLOUDS({
		if (FLidarPointCloudPoint* Point = Component->LineTraceSingle(Ray, Radius, bVisibleOnly))
		{
			const float DistanceSq = ((FVector)Point->Location - Origin).SizeSquared();
			if (DistanceSq < MinDistance)
			{
				MinDistance = DistanceSq;
				ClosestComponent = Component;
			}
		}
		});

	if (ClosestComponent)
	{
		ClosestComponent->ApplyColorToFirstPointByRay(NewColor, Ray, Radius, bVisibleOnly);
	}
}

void ULidarPointCloudBlueprintLibrary::ApplyColorToPointsByRay(UObject* WorldContextObject, FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly)
{
	ITERATE_CLOUDS({ Component->ApplyColorToPointsByRay(NewColor, Origin, Direction, Radius, bVisibleOnly); });
}

void ULidarPointCloudBlueprintLibrary::RemovePointsInSphere(UObject* WorldContextObject, FVector Center, float Radius, bool bVisibleOnly)
{
	ITERATE_CLOUDS({ Component->RemovePointsInSphere(Center, Radius, bVisibleOnly); });
}

void ULidarPointCloudBlueprintLibrary::RemovePointsInBox(UObject* WorldContextObject, FVector Center, FVector Extent, bool bVisibleOnly)
{
	ITERATE_CLOUDS({ Component->RemovePointsInBox(Center, Extent, bVisibleOnly); });
}

void ULidarPointCloudBlueprintLibrary::RemoveFirstPointByRay(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly)
{
	float MinDistance = FLT_MAX;
	ULidarPointCloudComponent* ClosestComponent = nullptr;

	const FLidarPointCloudRay Ray((FVector3f)Origin, (FVector3f)Direction);

	ITERATE_CLOUDS({
		if (FLidarPointCloudPoint* Point = Component->LineTraceSingle(Ray, Radius, bVisibleOnly))
		{
			const float DistanceSq = ((FVector)Point->Location - Origin).SizeSquared();
			if (DistanceSq < MinDistance)
			{
				MinDistance = DistanceSq;
				ClosestComponent = Component;
			}
		}
		});

	if (ClosestComponent)
	{
		ClosestComponent->RemoveFirstPointByRay(Ray, Radius, bVisibleOnly);
	}
}

void ULidarPointCloudBlueprintLibrary::RemovePointsByRay(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly)
{
	ITERATE_CLOUDS({ Component->RemovePointsByRay(Origin, Direction, Radius, bVisibleOnly); });
}

#undef ITERATE_CLOUDS

/*********************************************************************************************** ALidarClippingVolume */

ALidarClippingVolume::ALidarClippingVolume()
	: bEnabled(true)
	, Mode(ELidarClippingVolumeMode::ClipOutside)
	, Priority(0)
{
	bColored = true;
	BrushColor.R = 0;
	BrushColor.G = 128;
	BrushColor.B = 128;
	BrushColor.A = 255;

	GetBrushComponent()->SetMobility(EComponentMobility::Movable);

	SetActorScale3D(FVector(50));
}

#undef LOCTEXT_NAMESPACE
