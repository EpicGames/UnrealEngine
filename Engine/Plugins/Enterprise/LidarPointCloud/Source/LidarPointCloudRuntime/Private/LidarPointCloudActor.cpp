// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudActor.h"
#include "LidarPointCloudComponent.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/Pawn.h"
#include "LidarPointCloud.h"

ALidarPointCloudActor::ALidarPointCloudActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = PointCloudComponent = CreateDefaultSubobject<ULidarPointCloudComponent>(TEXT("PointCloudComponent"));
}

#if WITH_EDITOR
bool ALidarPointCloudActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (PointCloudComponent && PointCloudComponent->GetPointCloud())
	{
		Objects.Add(PointCloudComponent->GetPointCloud());
	}
	return true;
}
#endif

ULidarPointCloud* ALidarPointCloudActor::GetPointCloud() const
{
	return PointCloudComponent ? PointCloudComponent->GetPointCloud() : nullptr;
}

void ALidarPointCloudActor::SetPointCloud(ULidarPointCloud* PointCloud)
{
	if (PointCloudComponent)
	{
		PointCloudComponent->SetPointCloud(PointCloud);
	}
}