// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudComponent.h"
#include "Engine/CollisionProfile.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/Texture2D.h"
#include "PhysicsEngine/BodySetup.h"
#include "LidarPointCloud.h"

#if WITH_EDITOR
#include "Misc/MessageDialog.h"
#endif

#define IS_PROPERTY(Name) PropertyChangedEvent.MemberProperty->GetName().Equals(#Name)

ULidarPointCloudComponent::ULidarPointCloudComponent()
	: CustomMaterial(nullptr)
	, PointSize(1.0f)
	, ScalingMethod(ELidarPointCloudScalingMethod::PerNodeAdaptive)
	, GapFillingStrength(0)
	, ColorSource(ELidarPointCloudColorationMode::Data)
	, PointShape(ELidarPointCloudSpriteShape::Square)
	, PointOrientation(ELidarPointCloudSpriteOrientation::PreferFacingCamera)
	, ElevationColorBottom(FLinearColor::Red)
	, ElevationColorTop(FLinearColor::Green)
	, PointSizeBias(0.035f)
	, Saturation(FVector::OneVector)
	, Contrast(FVector::OneVector)
	, Gamma(FVector::OneVector)
	, Gain(FVector::OneVector)
	, Offset(FVector::ZeroVector)
	, ColorTint(FLinearColor::White)
	, IntensityInfluence(0.0f)
	, bUseFrustumCulling(true)
	, MinDepth(0)
	, MaxDepth(-1)
	, bDrawNodeBounds(false)
	, Material(nullptr)
	, OwningViewportClient(nullptr)
{
	PrimaryComponentTick.bCanEverTick = false;
	Mobility = EComponentMobility::Movable;

	CastShadow = false;
	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);

	static ConstructorHelpers::FObjectFinder<UMaterial> M_PointCloud(TEXT("/LidarPointCloud/Materials/M_LidarPointCloud"));
	BaseMaterial = M_PointCloud.Object;

	static ConstructorHelpers::FObjectFinder<UMaterial> M_PointCloud_Masked(TEXT("/LidarPointCloud/Materials/M_LidarPointCloud_Masked"));
	BaseMaterialMasked = M_PointCloud_Masked.Object;
}

FBoxSphereBounds ULidarPointCloudComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return PointCloud ? PointCloud->GetBounds().TransformBy(LocalToWorld) : USceneComponent::CalcBounds(LocalToWorld);
}

void ULidarPointCloudComponent::UpdateMaterial()
{
	// If the custom material is an instance, apply it directly...
	if (CustomMaterial && (Cast<UMaterialInstanceDynamic>(CustomMaterial) || Cast<UMaterialInstanceConstant>(CustomMaterial)))
	{
		Material = CustomMaterial;
	}
	// ... otherwise, create MID from it
	else
	{
		Material = UMaterialInstanceDynamic::Create(CustomMaterial ? CustomMaterial : PointShape != ELidarPointCloudSpriteShape::Square ? BaseMaterialMasked : BaseMaterial, nullptr);
	}

	ApplyRenderingParameters();
}

void ULidarPointCloudComponent::AttachPointCloudListener()
{
	if (PointCloud)
	{
		PointCloud->OnPointCloudRebuilt().AddUObject(this, &ULidarPointCloudComponent::OnPointCloudRebuilt);
		PointCloud->OnPointCloudCollisionUpdated().AddUObject(this, &ULidarPointCloudComponent::OnPointCloudCollisionUpdated);
		PointCloud->OnPointCloudNormalsUpdated().AddUObject(this, &ULidarPointCloudComponent::OnPointCloudNormalsUpdated);
	}
}

void ULidarPointCloudComponent::RemovePointCloudListener()
{
	if (PointCloud)
	{
		PointCloud->OnPointCloudRebuilt().RemoveAll(this);
		PointCloud->OnPointCloudCollisionUpdated().RemoveAll(this);
		PointCloud->OnPointCloudNormalsUpdated().RemoveAll(this);
	}
}

void ULidarPointCloudComponent::OnPointCloudRebuilt()
{
	MarkRenderStateDirty();
	UpdateBounds();
	UpdateMaterial();

	if (PointCloud)
	{
		if (ClassificationColors.Num() == 0)
		{
			for (uint8& Classification : PointCloud->GetClassificationsImported())
			{
				ClassificationColors.Emplace(Classification, FLinearColor::White);
			}
		}
	}
}

void ULidarPointCloudComponent::OnPointCloudCollisionUpdated()
{
	if (bPhysicsStateCreated)
	{
		RecreatePhysicsState();
	}

	MarkRenderStateDirty();
}

void ULidarPointCloudComponent::OnPointCloudNormalsUpdated()
{
	PointOrientation = ELidarPointCloudSpriteOrientation::PreferFacingNormal;
	MarkRenderStateDirty();
}

void ULidarPointCloudComponent::PostPointCloudSet()
{
	AttachPointCloudListener();

	if (PointCloud)
	{	
		for (uint8& Classification : PointCloud->GetClassificationsImported())
		{
			ClassificationColors.Emplace(Classification, FLinearColor::White);
		}
	}
}

#if WITH_EDITOR
void ULidarPointCloudComponent::SelectByConvexVolume(FConvexVolume ConvexVolume, bool bAdditive, bool bVisibleOnly)
{
	if(!PointCloud)
	{
		return;
	}
	
	const FMatrix InvTransform = GetComponentTransform().Inverse().ToMatrixNoScale().ConcatTranslation(-PointCloud->LocationOffset);

	for(FPlane& Plane : ConvexVolume.Planes)
	{
		Plane = Plane.TransformBy(InvTransform);
	}

	ConvexVolume.Init();

	PointCloud->SelectByConvexVolume(ConvexVolume, bAdditive, false, bVisibleOnly);
}

void ULidarPointCloudComponent::SelectBySphere(FSphere Sphere, bool bAdditive, bool bVisibleOnly)
{
	if(!PointCloud)
	{
		return;
	}
	
	const FMatrix InvTransform = GetComponentTransform().Inverse().ToMatrixNoScale().ConcatTranslation(-PointCloud->LocationOffset);
	
	PointCloud->SelectBySphere(Sphere.TransformBy(InvTransform), bAdditive, false, bVisibleOnly);
}

void ULidarPointCloudComponent::HideSelected()
{
	if(PointCloud)
	{
		PointCloud->HideSelected();
	}
}

void ULidarPointCloudComponent::DeleteSelected()
{
	if(PointCloud)
	{
		PointCloud->DeleteSelected();
	}
}

void ULidarPointCloudComponent::InvertSelection()
{
	if(PointCloud)
	{
		PointCloud->InvertSelection();
	}
}

int64 ULidarPointCloudComponent::NumSelectedPoints()
{
	return PointCloud ? PointCloud->NumSelectedPoints() : 0;
}

void ULidarPointCloudComponent::ClearSelection()
{
	if(PointCloud)
	{
		PointCloud->ClearSelection();
	}
}
#endif // WITH_EDITOR

void ULidarPointCloudComponent::SetPointCloud(ULidarPointCloud *InPointCloud)
{
	if (PointCloud != InPointCloud)
	{
		RemovePointCloudListener();
		PointCloud = InPointCloud;		
		PostPointCloudSet();
		OnPointCloudRebuilt();
	}
}

void ULidarPointCloudComponent::SetPointShape(ELidarPointCloudSpriteShape NewPointShape)
{
	PointShape = NewPointShape;
	UpdateMaterial();
}

void ULidarPointCloudComponent::ApplyRenderingParameters()
{
	if (UMaterialInstanceDynamic* DynMaterial = Cast<UMaterialInstanceDynamic>(Material))
	{
		DynMaterial->SetVectorParameterValue("PC__Gain", FVector(Gain.X, Gain.Y, Gain.Z) * Gain.W);
		DynMaterial->SetScalarParameterValue("PC__GapFillerFactor", GapFillingStrength);
	}
}

void ULidarPointCloudComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	ULidarPointCloudComponent* This = CastChecked<ULidarPointCloudComponent>(InThis);
	Super::AddReferencedObjects(This, Collector);
}

void ULidarPointCloudComponent::PostLoad()
{
	Super::PostLoad();
	AttachPointCloudListener();
	UpdateMaterial();
}

void ULidarPointCloudComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial)
{
	// If the material cannot be used with LidarPointClouds, then warn and cancel
	if (InMaterial && !InMaterial->CheckMaterialUsage(MATUSAGE_LidarPointCloud))
	{
#if WITH_EDITOR
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LidarPointCloud", "Error_Material_PointCloud", "Can't use the specified material because it has not been compiled with bUsedWithLidarPointCloud."));
#endif
		return;
	}

	CustomMaterial = InMaterial;
	OnPointCloudRebuilt();
}

UBodySetup* ULidarPointCloudComponent::GetBodySetup()
{	
	return PointCloud ? PointCloud->GetBodySetup() : nullptr;
}

#if WITH_EDITOR
void ULidarPointCloudComponent::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange)
	{
		if (PropertyThatWillChange->GetName().Equals("PointCloud"))
		{
			RemovePointCloudListener();
		}
	}
}

void ULidarPointCloudComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty)
	{
		if (IS_PROPERTY(PointCloud))
		{
			PostPointCloudSet();
		}

		if (IS_PROPERTY(CustomMaterial))
		{
			SetMaterial(0, CustomMaterial);
		}

		if (IS_PROPERTY(Gain))
		{
			ApplyRenderingParameters();
		}

		if (IS_PROPERTY(GapFillingStrength))
		{
			ApplyRenderingParameters();
		}

		if (IS_PROPERTY(PointShape))
		{
			SetPointShape(PointShape);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void FLidarPointCloudComponentRenderParams::UpdateFromComponent(ULidarPointCloudComponent* Component)
{
	MinDepth = Component->MinDepth;
	MaxDepth = Component->MaxDepth;

	BoundsScale = Component->BoundsScale;
	BoundsSize = (FVector3f)Component->GetPointCloud()->GetBounds().GetSize();

	// Make sure to apply minimum bounds size
	BoundsSize.X = FMath::Max(BoundsSize.X, 0.001f);
	BoundsSize.Y = FMath::Max(BoundsSize.Y, 0.001f);
	BoundsSize.Z = FMath::Max(BoundsSize.Z, 0.001f);

	LocationOffset = (FVector3f)Component->GetPointCloud()->LocationOffset;
	ComponentScale = Component->GetComponentScale().GetAbsMax();

	PointSize = Component->PointSize;
	PointSizeBias = Component->PointSizeBias;
	GapFillingStrength = Component->GapFillingStrength;

	bOwnedByEditor = Component->IsOwnedByEditor();
	bDrawNodeBounds = Component->bDrawNodeBounds;
	bShouldRenderFacingNormals = Component->ShouldRenderFacingNormals();
	bUseFrustumCulling = Component->bUseFrustumCulling;
	
	ScalingMethod = Component->ScalingMethod;

	// Per-point scaling is currently unavailable in Ray Tracing
	if(ScalingMethod == ELidarPointCloudScalingMethod::PerPoint && GetDefault<ULidarPointCloudSettings>()->bEnableLidarRayTracing)
	{
		ScalingMethod = ELidarPointCloudScalingMethod::PerNodeAdaptive;
	}

	ColorSource = Component->ColorSource;
	PointShape = Component->GetPointShape();

	Offset = (FVector4f)Component->Offset;
	Contrast = (FVector4f)Component->Contrast;
	Saturation = (FVector4f)Component->Saturation;
	Gamma = (FVector4f)Component->Gamma;
	ColorTint = FVector3f(Component->ColorTint);
	IntensityInfluence = Component->IntensityInfluence;

	ClassificationColors = Component->ClassificationColors;
	ElevationColorBottom = Component->ElevationColorBottom;
	ElevationColorTop = Component->ElevationColorTop;

	Material = Component->GetMaterial(0);
}
