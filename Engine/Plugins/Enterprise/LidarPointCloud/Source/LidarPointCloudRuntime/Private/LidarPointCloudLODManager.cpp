// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudLODManager.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudComponent.h"
#include "LidarPointCloudOctree.h"
#include "LidarPointCloudSettings.h"
#include "Rendering/LidarPointCloudRenderBuffers.h"
#include "Engine/LocalPlayer.h"
#include "Kismet/GameplayStatics.h"
#include "Async/Async.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeTryLock.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "UnrealClient.h"

#if WITH_EDITOR
#include "Settings/EditorStyleSettings.h"
#include "EditorViewportClient.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Node Selection"), STAT_NodeSelection, STATGROUP_LidarPointCloud)
DECLARE_CYCLE_STAT(TEXT("Node Processing"), STAT_NodeProcessing, STATGROUP_LidarPointCloud)
DECLARE_CYCLE_STAT(TEXT("Render Data Update"), STAT_UpdateRenderData, STATGROUP_LidarPointCloud)
DECLARE_CYCLE_STAT(TEXT("Node Streaming"), STAT_NodeStreaming, STATGROUP_LidarPointCloud);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Point Count [thousands]"), STAT_PointCountTotal, STATGROUP_LidarPointCloud)
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Point Budget"), STAT_PointBudget, STATGROUP_LidarPointCloud)
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Visible Points"), STAT_PointCount, STATGROUP_LidarPointCloud)
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Loaded Nodes"), STAT_LoadedNodes, STATGROUP_LidarPointCloud)

static TAutoConsoleVariable<int32> CVarLidarPointBudget(
	TEXT("r.LidarPointBudget"),
	0,
	TEXT("If set to > 0, this will overwrite the Target FPS setting, and apply a fixed budget.\n")
	TEXT("Determines the maximum number of points to be visible on the screen.\n")
	TEXT("Higher values will produce better image quality, but will require faster hardware."),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarLidarScreenCenterImportance(
	TEXT("r.LidarScreenCenterImportance"),
	0.0f,
	TEXT("Determines the preference towards selecting nodes closer to screen center\n")
	TEXT("with larger values giving more priority towards screen center.\n")
	TEXT("Usefulf for VR, where edge vision is blurred anyway.\n")
	TEXT("0 to disable."),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarBaseLODImportance(
	TEXT("r.LidarBaseLODImportance"),
	0.1f,
	TEXT("Determines the importance of selecting at least the base LOD of far assets.\n")
	TEXT("Increase it, if you're experiencing actor 'popping'.\n")
	TEXT("0 to use purely screensize-driven algorithm."),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarTargetFPS(
	TEXT("r.LidarTargetFPS"),
	59.0f,
	TEXT("The LOD system will continually adjust the quality of the assets to maintain\n")
	TEXT("the specified target FPS."),
	ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarLidarIncrementalBudget(
	TEXT("r.LidarIncrementalBudget"),
	false,
	TEXT("If enabled, the point budget will automatically increase whenever the\n")
	TEXT("camera's location and orientation remain unchanged."),
	ECVF_Scalability);

FLidarPointCloudViewData::FLidarPointCloudViewData(bool bCompute)
	: bValid(false)
	, ViewOrigin(FVector::ZeroVector)
	, ViewDirection(FVector::ForwardVector)
	, ScreenSizeFactor(0)
	, bSkipMinScreenSize(false)
	, bPIE(false)
	, bHasFocus(false)
{
	if (bCompute)
	{
		Compute();
	}
}

void FLidarPointCloudViewData::Compute()
{
	bool bForceSkipLocalPlayer = false;

#if WITH_EDITOR
	bForceSkipLocalPlayer = GIsEditor && GEditor && GEditor->bIsSimulatingInEditor;
#endif

	// Attempt to get the first local player's viewport
	if (GEngine && !bForceSkipLocalPlayer)
	{
		ULocalPlayer* const LP = GEngine->FindFirstLocalPlayerFromControllerId(0);
		if (LP && LP->ViewportClient)
		{
			FSceneViewProjectionData ProjectionData;
			if (LP->GetProjectionData(LP->ViewportClient->Viewport, ProjectionData,
				GEngine->IsStereoscopic3D() ? EStereoscopicEye::eSSE_LEFT_EYE : EStereoscopicEye::eSSE_MONOSCOPIC))
			{
				ViewOrigin = ProjectionData.ViewOrigin;
				FMatrix ViewRotationMatrix = ProjectionData.ViewRotationMatrix;
				if (!ViewRotationMatrix.GetOrigin().IsNearlyZero(0.0f))
				{
					ViewOrigin += ViewRotationMatrix.InverseTransformPosition(FVector::ZeroVector);
					ViewRotationMatrix = ViewRotationMatrix.RemoveTranslation();
				}

				FMatrix ViewMatrix = FTranslationMatrix(-ViewOrigin) * ViewRotationMatrix;
				ViewDirection = ViewMatrix.GetColumn(2);
				FMatrix ProjectionMatrix = AdjustProjectionMatrixForRHI(ProjectionData.ProjectionMatrix);

				ScreenSizeFactor = FMath::Square(FMath::Max(0.5f * ProjectionMatrix.M[0][0], 0.5f * ProjectionMatrix.M[1][1]));

				// Skip SS check, if not in the projection view nor game world
				bSkipMinScreenSize = (ProjectionMatrix.M[3][3] >= 1.0f) && !LP->GetWorld()->IsGameWorld();
				GetViewFrustumBounds(ViewFrustum, ViewMatrix * ProjectionMatrix, false);

				bHasFocus = LP->ViewportClient->Viewport->HasFocus();

				bValid = true;
			}
		}
	}

#if WITH_EDITOR
	bPIE = false;
	if (GIsEditor && GEditor && GEditor->GetActiveViewport())
	{
		bPIE = GEditor->GetActiveViewport() == GEditor->GetPIEViewport();
		
		// PIE needs a different computation method
		if (!bValid && !bPIE)
		{
			ComputeFromEditorViewportClient(GEditor->GetActiveViewport()->GetClient());
		}

		// Simulating counts as PIE for the purpose of LOD calculation
		bPIE |= GEditor->bIsSimulatingInEditor;
	}
#endif
}

bool FLidarPointCloudViewData::ComputeFromEditorViewportClient(FViewportClient* ViewportClient)
{
#if WITH_EDITOR
	if (FEditorViewportClient* Client = (FEditorViewportClient*)ViewportClient)
	{
		if (Client->Viewport && Client->Viewport->GetSizeXY() != FIntPoint::ZeroValue)
		{
			FSceneViewFamily::ConstructionValues CVS(nullptr, nullptr, FEngineShowFlags(EShowFlagInitMode::ESFIM_Game));
			CVS.SetTime(FGameTime());
			FSceneViewFamilyContext ViewFamily(CVS);
			FSceneView* View = Client->CalcSceneView(&ViewFamily);

			const FMatrix& ProjectionMatrix = View->ViewMatrices.GetViewToClip();
			ScreenSizeFactor = FMath::Square(FMath::Max(0.5f * ProjectionMatrix.M[0][0], 0.5f * ProjectionMatrix.M[1][1]));
			ViewOrigin = View->ViewMatrices.GetViewOrigin();
			ViewDirection = View->GetViewDirection();
			ViewFrustum = View->ViewFrustum;
			bSkipMinScreenSize = !View->bIsGameView && !View->IsPerspectiveProjection();
			bHasFocus = Client->Viewport->HasFocus();

			bValid = true;

			return true;
		}
	}
#endif
	return false;
}

int32 FLidarPointCloudTraversalOctree::GetVisibleNodes(TArray<FLidarPointCloudTraversalOctreeNodeSizeData>& NodeSizeData, const FLidarPointCloudViewData* ViewData, const int32& ProxyIndex, const FLidarPointCloudNodeSelectionParams& SelectionParams)
{
	const int32 CurrentNodeCount = NodeSizeData.Num();

	// Only process, if the asset is visible
	if (ViewData->ViewFrustum.IntersectBox(GetCenter(), GetExtent()))
	{
		const float BoundsScaleSq = SelectionParams.BoundsScale * SelectionParams.BoundsScale;
		const float BaseLODImportance = FMath::Max(0.0f, CVarBaseLODImportance.GetValueOnAnyThread());

		TQueue<FLidarPointCloudTraversalOctreeNode*> Nodes;
		FLidarPointCloudTraversalOctreeNode* CurrentNode = nullptr;
		Nodes.Enqueue(&Root);
		while (Nodes.Dequeue(CurrentNode))
		{
			// Reset selection flag
			CurrentNode->bSelected = false;

			// Update number of visible points, if needed
			CurrentNode->DataNode->UpdateNumVisiblePoints();

			const FVector3f NodeExtent = Extents[CurrentNode->Depth] * SelectionParams.BoundsScale;

			bool bFullyContained = true;

			if (SelectionParams.bUseFrustumCulling && (CurrentNode->Depth == 0 || !CurrentNode->bFullyContained) && !ViewData->ViewFrustum.IntersectBox((FVector)CurrentNode->Center, (FVector)NodeExtent, bFullyContained))
			{
				continue;
			}
			
			// Only process this node if it has any visible points - do not use continue; as the children may still contain visible points!
			if (CurrentNode->DataNode->GetNumVisiblePoints() > 0 && CurrentNode->Depth >= SelectionParams.MinDepth)
			{
				float ScreenSizeSq = 0;

				FVector3f VectorToNode = CurrentNode->Center - (FVector3f)ViewData->ViewOrigin;
				const float DistSq = VectorToNode.SizeSquared();
				const float AdjustedRadiusSq = RadiiSq[CurrentNode->Depth] * BoundsScaleSq;

				// Make sure to show at least the minimum depth for each visible asset
				if (CurrentNode->Depth == SelectionParams.MinDepth)
				{
					// Add screen size to maintain hierarchy
					ScreenSizeSq = BaseLODImportance + ViewData->ScreenSizeFactor * AdjustedRadiusSq / FMath::Max(1.0f, DistSq);
				}
				else
				{
					// If the camera is within this node's bounds, it should always be qualified for rendering
					if (DistSq <= AdjustedRadiusSq)
					{
						// Subtract Depth to maintain hierarchy 
						ScreenSizeSq = 1000 - CurrentNode->Depth;
					}
					else
					{
						ScreenSizeSq = ViewData->ScreenSizeFactor * AdjustedRadiusSq / FMath::Max(1.0f, DistSq);

						// Check for minimum screen size
						if (!ViewData->bSkipMinScreenSize && ScreenSizeSq < SelectionParams.MinScreenSize)
						{
							continue;
						}

						// Add optional preferential selection for nodes closer to the screen center
						if (SelectionParams.ScreenCenterImportance > 0)
						{
							VectorToNode.Normalize();
							float Dot = FVector3f::DotProduct((FVector3f)ViewData->ViewDirection, VectorToNode);

							ScreenSizeSq = FMath::Lerp(ScreenSizeSq, ScreenSizeSq * Dot, SelectionParams.ScreenCenterImportance);
						}
					}
				}

				NodeSizeData.Emplace(CurrentNode, ScreenSizeSq, ProxyIndex);
			}

			if (SelectionParams.MaxDepth < 0 || CurrentNode->Depth < SelectionParams.MaxDepth)
			{
				for (FLidarPointCloudTraversalOctreeNode& Child : CurrentNode->Children)
				{
					Child.bFullyContained = bFullyContained;
					Nodes.Enqueue(&Child);
				}
			}
		}
	}

	return NodeSizeData.Num() - CurrentNodeCount;
}

/** Calculates the correct point budget to use for current frame */
uint32 GetPointBudget(int64 NumPointsInFrustum)
{
	constexpr int32 NumFramesToAcumulate = 30;
	constexpr int32 DeltaBudgetDeadzone = 10000;

	static int64 CurrentPointBudget = 0;
	static int64 LastDynamicPointBudget = 0;
	static bool bLastFrameIncremental = false;
	static FLidarPointCloudViewData LastViewData;
	static TArray<float> AcumulatedFrameTime;
	static double CurrentRealTime = 0;
	static double LastRealTime = 0;
	static double RealDeltaTime = 0;

	CurrentRealTime = FPlatformTime::Seconds();
	RealDeltaTime = CurrentRealTime - LastRealTime;
	LastRealTime = CurrentRealTime;

	if (AcumulatedFrameTime.Num() == 0)
	{
		AcumulatedFrameTime.Reserve(NumFramesToAcumulate + 1);
	}

	const FLidarPointCloudViewData ViewData(true);

	if (!LastViewData.bValid)
	{
		LastViewData = ViewData;
	}

	bool bUseIncrementalBudget = CVarLidarIncrementalBudget.GetValueOnAnyThread();
	const int32 ManualPointBudget = CVarLidarPointBudget.GetValueOnAnyThread();

	if (bUseIncrementalBudget && ViewData.ViewOrigin.Equals(LastViewData.ViewOrigin) && ViewData.ViewDirection.Equals(LastViewData.ViewDirection))
	{
		CurrentPointBudget += 500000;
		bLastFrameIncremental = true;
	}
	else
	{
		// Check if the point budget is manually set
		if (ManualPointBudget > 0)
		{
			CurrentPointBudget = ManualPointBudget;
		}
		else
		{
			CurrentPointBudget = LastDynamicPointBudget;

			// Do not recalculate if just exiting incremental budget, to avoid spikes
			if (!bLastFrameIncremental)
			{
				if (AcumulatedFrameTime.Add(RealDeltaTime) == NumFramesToAcumulate)
				{
					AcumulatedFrameTime.RemoveAt(0);
				}

				const float MaxTickRate = GEngine->GetMaxTickRate(0.001f, false);
				const float RequestedTargetFPS = CVarTargetFPS.GetValueOnAnyThread();
				const float TargetFPS = GEngine->bUseFixedFrameRate ? GEngine->FixedFrameRate : (MaxTickRate > 0 ? FMath::Min(RequestedTargetFPS, MaxTickRate) : RequestedTargetFPS);

				// The -0.5f is to prevent the system treating values as unachievable (as the frame time is usually just under)
				const float AdjustedTargetFPS = FMath::Max(TargetFPS - 0.5f, 1.0f);

				TArray<float> CurrentFrameTimes = AcumulatedFrameTime;
				CurrentFrameTimes.Sort();
				const float AvgFrameTime = CurrentFrameTimes[CurrentFrameTimes.Num() / 2];

				const int32 DeltaBudget = (1 / AdjustedTargetFPS - AvgFrameTime) * 10000000 * (GEngine->bUseFixedFrameRate ? 4 : 1);;

				// Prevent constant small fluctuations, unless using fixed frame rate
				if (GEngine->bUseFixedFrameRate || FMath::Abs(DeltaBudget) > DeltaBudgetDeadzone)
				{
					// Not having enough points in frustum to fill the requested budget would otherwise continually increase the value
					if (DeltaBudget < 0 || NumPointsInFrustum >= CurrentPointBudget)
					{
						CurrentPointBudget += DeltaBudget;
					}
				}
			}
		}

		bLastFrameIncremental = false;
	}

	// Just in case
	if (ManualPointBudget == 0)
	{
		CurrentPointBudget = FMath::Clamp(CurrentPointBudget, 350000LL, 100000000LL);
	}

	if (!bUseIncrementalBudget)
	{
		LastDynamicPointBudget = CurrentPointBudget;
	}

	LastViewData = ViewData;

	return CurrentPointBudget;
}

FLidarPointCloudLODManager::FLidarPointCloudLODManager()
	: NumPointsInFrustum(0)
{
}

void FLidarPointCloudLODManager::Tick(float DeltaTime)
{
	// Skip processing, if a previous one is still going
	if (bProcessing)
	{
		return;
	}

	bProcessing = true;

	Time += DeltaTime;
	
	LastPointBudget = GetPointBudget(NumPointsInFrustum.GetValue());

	SET_DWORD_STAT(STAT_PointBudget, LastPointBudget);

	PrepareProxies();

	Async(EAsyncExecution::ThreadPool, [this, CurrentRegisteredProxies = RegisteredProxies, PointBudget = LastPointBudget, ClippingVolumes = GetClippingVolumes()]
	{
		NumPointsInFrustum.Set(ProcessLOD(CurrentRegisteredProxies, Time, PointBudget, ClippingVolumes));
		bProcessing = false;
	});
}

TStatId FLidarPointCloudLODManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(LidarPointCloudLODManager, STATGROUP_Tickables);
}

void FLidarPointCloudLODManager::RegisterProxy(ULidarPointCloudComponent* Component, TWeakPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper)
{
	if (IsValid(Component))
	{
		static FCriticalSection ProxyLock;

		if (TSharedPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapperShared = SceneProxyWrapper.Pin())
		{
			FScopeLock Lock(&ProxyLock);
			Get().RegisteredProxies.Emplace(Component, SceneProxyWrapper);
			Get().ForceProcessLOD();
		}
	}
}

void FLidarPointCloudLODManager::RefreshLOD()
{
	Get().ForceProcessLOD();
}

int64 FLidarPointCloudLODManager::ProcessLOD(const TArray<FLidarPointCloudLODManager::FRegisteredProxy>& InRegisteredProxies, const float CurrentTime, const uint32 PointBudget, const TArray<FLidarPointCloudClippingVolumeParams>& ClippingVolumes)
{
	struct FSelectionFilterParams
	{
		float MinScreenSize;
		uint32 NumNodes;
	};

	static TArray<FSelectionFilterParams> SelectionFilterParams;
	{
		const int32 DeltaSize = InRegisteredProxies.Num() - SelectionFilterParams.Num();
		if (DeltaSize > 0)
		{
			SelectionFilterParams.AddZeroed(DeltaSize);
		}
	}
	
	const bool bUseRayTracing = GetDefault<ULidarPointCloudSettings>()->bEnableLidarRayTracing;

	uint32 TotalPointsSelected = 0;
	int64 NewNumPointsInFrustum = 0;

	TArray<TArray<FLidarPointCloudTraversalOctreeNode*>> SelectedNodesData;

	// Node selection
	{
		SCOPE_CYCLE_COUNTER(STAT_NodeSelection);

		const float ScreenCenterImportance = CVarLidarScreenCenterImportance.GetValueOnAnyThread();
		int32 NumSelectedNodes = 0;

		TArray<FLidarPointCloudTraversalOctreeNodeSizeData> NodeSizeData;

		for (int32 i = 0; i < InRegisteredProxies.Num(); ++i)
		{
			const FLidarPointCloudLODManager::FRegisteredProxy& RegisteredProxy = InRegisteredProxies[i];

			// Acquire a Shared Pointer from the Weak Pointer and check that it references a valid object
			if (TSharedPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper = RegisteredProxy.SceneProxyWrapper.Pin())
			{
#if WITH_EDITOR
				// Avoid doubling the point allocation of the same asset (once in Editor world and once in PIE world)
				if (RegisteredProxy.bSkip)
				{
					continue;
				}
#endif

				// Attempt to get a data lock and check if the traversal octree is still valid
				FScopeTryLock OctreeLock(&RegisteredProxy.Octree->DataLock);
				if (OctreeLock.IsLocked() && RegisteredProxy.TraversalOctree->bValid)
				{
					FLidarPointCloudNodeSelectionParams SelectionParams;
					SelectionParams.MinScreenSize = SelectionFilterParams[i].MinScreenSize;
					SelectionParams.ScreenCenterImportance = ScreenCenterImportance;
					SelectionParams.MinDepth = RegisteredProxy.ComponentRenderParams.MinDepth;
					SelectionParams.MaxDepth = RegisteredProxy.ComponentRenderParams.MaxDepth;
					SelectionParams.BoundsScale = RegisteredProxy.ComponentRenderParams.BoundsScale;
					SelectionParams.bUseFrustumCulling = RegisteredProxy.ComponentRenderParams.bUseFrustumCulling && !bUseRayTracing;
					SelectionParams.ClippingVolumes = RegisteredProxy.ComponentRenderParams.bOwnedByEditor ? nullptr : &ClippingVolumes; // Ignore clipping if in editor viewport

					// Append visible nodes
					SelectionFilterParams[i].NumNodes = RegisteredProxy.TraversalOctree->GetVisibleNodes(NodeSizeData, &RegisteredProxy.ViewData, i, SelectionParams);
				}
			}
		}

		// Sort Nodes
		Algo::Sort(NodeSizeData, [](const FLidarPointCloudTraversalOctreeNodeSizeData& A, const FLidarPointCloudTraversalOctreeNodeSizeData& B) { return A.Size > B.Size; });

		TArray<float> NewMinScreenSizes;
		NewMinScreenSizes.AddZeroed(InRegisteredProxies.Num());

		// Limit nodes using specified Point Budget
		SelectedNodesData.AddDefaulted(InRegisteredProxies.Num());
		for (FLidarPointCloudTraversalOctreeNodeSizeData& Element : NodeSizeData)
		{
			const uint32 NumPoints = Element.Node->DataNode->GetNumVisiblePoints();
			const uint32 NewNumPointsSelected = TotalPointsSelected + NumPoints;
			NewNumPointsInFrustum += NumPoints;

			if (NewNumPointsSelected <= PointBudget)
			{
				NewMinScreenSizes[Element.ProxyIndex] = FMath::Max(Element.Size, 0.0f);
				SelectedNodesData[Element.ProxyIndex].Add(Element.Node);
				TotalPointsSelected = NewNumPointsSelected;
				Element.Node->bSelected = true;
				--SelectionFilterParams[Element.ProxyIndex].NumNodes;
				++NumSelectedNodes;
			}
		}

		for (int32 i = 0; i < InRegisteredProxies.Num(); ++i)
		{
			// If point budget is saturated, apply new Min Screen Sizes
			// Otherwise, decrease Min Screen Sizes, to allow for more points
			SelectionFilterParams[i].MinScreenSize = SelectionFilterParams[i].NumNodes > 0 ? NewMinScreenSizes[i] : (SelectionFilterParams[i].MinScreenSize * 0.9f);
		}

		SET_DWORD_STAT(STAT_PointCount, TotalPointsSelected);
	}

	// Used to pass render data updates to render thread
	TArray<FLidarPointCloudProxyUpdateData> ProxyUpdateData;

	// Holds a per-octree list of nodes to stream-in or extend
	TMultiMap<FLidarPointCloudOctree*, TArray<FLidarPointCloudOctreeNode*>> OctreeStreamingMap;

	// Process Nodes
	{
		SCOPE_CYCLE_COUNTER(STAT_NodeProcessing);

		const bool bUseStaticBuffers = GetDefault<ULidarPointCloudSettings>()->bUseFastRendering;

		for (int32 i = 0; i < SelectedNodesData.Num(); ++i)
		{
			const FLidarPointCloudLODManager::FRegisteredProxy& RegisteredProxy = InRegisteredProxies[i];

			// Attempt to get a data lock and check if the traversal octree is still valid
			FScopeTryLock OctreeLock(&RegisteredProxy.Octree->DataLock);
			if (OctreeLock.IsLocked() && RegisteredProxy.TraversalOctree->bValid)
			{
				FLidarPointCloudProxyUpdateData UpdateData;
				UpdateData.SceneProxyWrapper = RegisteredProxy.SceneProxyWrapper;
				UpdateData.NumElements = 0;
				UpdateData.VDMultiplier = RegisteredProxy.TraversalOctree->ReversedVirtualDepthMultiplier;
				UpdateData.RootCellSize = RegisteredProxy.Octree->GetRootCellSize();
				UpdateData.ClippingVolumes = ClippingVolumes;
				UpdateData.bUseStaticBuffers = bUseStaticBuffers && !RegisteredProxy.Octree->IsOptimizedForDynamicData();
				UpdateData.bUseRayTracing = bUseRayTracing;
				UpdateData.RenderParams = RegisteredProxy.ComponentRenderParams;

#if !(UE_BUILD_SHIPPING)
				const bool bDrawNodeBounds = RegisteredProxy.ComponentRenderParams.bDrawNodeBounds;
				if (bDrawNodeBounds)
				{
					UpdateData.Bounds.Reset(SelectedNodesData[i].Num());
				}
#endif
				const bool bCalculateVirtualDepth = RegisteredProxy.ComponentRenderParams.PointSize > 0;

				TArray<float> LocalLevelWeights;
				TArray<float>* LevelWeightsPtr = &RegisteredProxy.TraversalOctree->LevelWeights;
				float PointSizeBias = RegisteredProxy.ComponentRenderParams.PointSizeBias;

				// Only calculate if needed
				if (bCalculateVirtualDepth && (RegisteredProxy.ComponentRenderParams.ScalingMethod == ELidarPointCloudScalingMethod::PerNodeAdaptive || RegisteredProxy.ComponentRenderParams.ScalingMethod == ELidarPointCloudScalingMethod::PerPoint))
				{
					RegisteredProxy.TraversalOctree->CalculateLevelWeightsForSelectedNodes(LocalLevelWeights);
					LevelWeightsPtr = &LocalLevelWeights;
					PointSizeBias = 0;
				}

				// Queue nodes to be streamed
				TArray<FLidarPointCloudOctreeNode*>& NodesToStream = OctreeStreamingMap.FindOrAdd(RegisteredProxy.Octree);
				NodesToStream.Reserve(NodesToStream.Num() + SelectedNodesData[i].Num());
				for (FLidarPointCloudTraversalOctreeNode* Node : SelectedNodesData[i])
				{
					NodesToStream.Add(Node->DataNode);

					if (Node->DataNode->HasData())
					{
						// Only calculate if needed
                        if (bCalculateVirtualDepth)
                        {
							Node->CalculateVirtualDepth(*LevelWeightsPtr, PointSizeBias);
                        }
						
						const uint32 NumVisiblePoints = Node->DataNode->GetNumVisiblePoints();
						UpdateData.NumElements += NumVisiblePoints;
						UpdateData.SelectedNodes.Emplace(bCalculateVirtualDepth ? Node->VirtualDepth : 0, NumVisiblePoints, Node->DataNode);
#if !(UE_BUILD_SHIPPING)
						if (bDrawNodeBounds)
						{
							const FVector3f Extent = RegisteredProxy.TraversalOctree->Extents[Node->Depth];
							UpdateData.Bounds.Emplace(Node->Center - Extent, Node->Center + Extent);
						}
#endif
					}
				}

				// Only calculate if needed
				if (bCalculateVirtualDepth && RegisteredProxy.ComponentRenderParams.ScalingMethod == ELidarPointCloudScalingMethod::PerPoint)
				{
					RegisteredProxy.TraversalOctree->CalculateVisibilityStructure(UpdateData.TreeStructure);
				}

				ProxyUpdateData.Add(UpdateData);
			}
		}
	}

	// Perform data streaming in a separate thread
	Async(EAsyncExecution::ThreadPool, [OctreeStreamingMap = MoveTemp(OctreeStreamingMap), CurrentTime]() mutable
	{
		SCOPE_CYCLE_COUNTER(STAT_NodeStreaming);

		int32 LoadedNodes = 0;
		for (TPair<FLidarPointCloudOctree*, TArray<FLidarPointCloudOctreeNode*>>& OctreeStreamingData : OctreeStreamingMap)
		{
			FScopeTryLock OctreeLock(&OctreeStreamingData.Key->DataLock);
			if (OctreeLock.IsLocked())
			{
				OctreeStreamingData.Key->StreamNodes(OctreeStreamingData.Value, CurrentTime);
				LoadedNodes += OctreeStreamingData.Key->GetNumNodesInUse();
			}
		}

		SET_DWORD_STAT(STAT_LoadedNodes, LoadedNodes);
	});

	// Update Render Data
	if (TotalPointsSelected > 0)
	{
		ENQUEUE_RENDER_COMMAND(ProcessLidarPointCloudLOD)([ProxyUpdateData](FRHICommandListImmediate& RHICmdList) mutable
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateRenderData);

			uint32 MaxPointsPerNode = 0;

			const double ProcessingTime = FPlatformTime::Seconds();
			const bool bUseRenderDataSmoothing = GetDefault<ULidarPointCloudSettings>()->bUseRenderDataSmoothing;
			const float RenderDataSmoothingMaxFrametime = GetDefault<ULidarPointCloudSettings>()->RenderDataSmoothingMaxFrametime;

			// Iterate over proxies and, if valid, update their data
			for (FLidarPointCloudProxyUpdateData& UpdateData : ProxyUpdateData)
			{
				// Check for proxy's validity, in case it has been destroyed since the update was issued
				if (TSharedPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper = UpdateData.SceneProxyWrapper.Pin())
				{
					if (SceneProxyWrapper->Proxy)
					{
						for (FLidarPointCloudProxyUpdateDataNode& Node : UpdateData.SelectedNodes)
						{
							if (Node.BuildDataCache(UpdateData.bUseStaticBuffers, UpdateData.bUseRayTracing))
							{
								MaxPointsPerNode = FMath::Max(MaxPointsPerNode, (uint32)Node.NumVisiblePoints);
							}

							// Split building render data across multiple frames, to avoid stuttering
							if (bUseRenderDataSmoothing && (FPlatformTime::Seconds() - ProcessingTime > RenderDataSmoothingMaxFrametime))
							{
								break;
							}
						}

						SceneProxyWrapper->Proxy->UpdateRenderData(UpdateData);
					}
				}
			}

			if (MaxPointsPerNode > GLidarPointCloudIndexBuffer.GetCapacity())
			{
				GLidarPointCloudIndexBuffer.Resize(MaxPointsPerNode);
			}
		});
	}

	return NewNumPointsInFrustum;
}

void FLidarPointCloudLODManager::ForceProcessLOD()
{
	if(IsInGameThread())
	{
		PrepareProxies();
		ProcessLOD(RegisteredProxies, Time, LastPointBudget, GetClippingVolumes());
	}
}

void FLidarPointCloudLODManager::PrepareProxies()
{
	FLidarPointCloudViewData ViewData(true);

	const bool bPrioritizeActiveViewport = GetDefault<ULidarPointCloudSettings>()->bPrioritizeActiveViewport;

	// Contains the total number of points contained by all assets (including invisible and culled)
	int64 TotalPointCount = 0;

	// Prepare proxies
	for (int32 i = 0; i < RegisteredProxies.Num(); ++i)
	{
		FRegisteredProxy& RegisteredProxy = RegisteredProxies[i];
		bool bValidProxy = false;

		// Acquire a Shared Pointer from the Weak Pointer and check that it references a valid object
		if (TSharedPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper = RegisteredProxy.SceneProxyWrapper.Pin())
		{
			if (ULidarPointCloudComponent* Component = RegisteredProxy.Component.Get())
			{
				if (ULidarPointCloud* PointCloud = RegisteredProxy.PointCloud.Get())
				{
					// Just in case
					if (Component->GetPointCloud() == PointCloud)
					{
#if WITH_EDITOR
						// Avoid doubling the point allocation of the same asset (once in Editor world and once in PIE world)
						RegisteredProxy.bSkip = ViewData.bPIE && Component->GetWorld() && Component->GetWorld()->WorldType == EWorldType::Type::Editor;
#endif

						// Check if the component's transform has changed, and invalidate the Traversal Octree if so
						const FTransform Transform = Component->GetComponentTransform();
						if (!RegisteredProxy.LastComponentTransform.Equals(Transform))
						{
							RegisteredProxy.TraversalOctree->bValid = false;
							RegisteredProxy.LastComponentTransform = Transform;
						}

						// Re-initialize the traversal octree, if needed
						if (!RegisteredProxy.TraversalOctree->bValid)
						{
							// Update asset reference
							RegisteredProxy.PointCloud = PointCloud;

							// Recreate the Traversal Octree
							RegisteredProxy.TraversalOctree = MakeShareable(new FLidarPointCloudTraversalOctree(&PointCloud->Octree, Component->GetComponentTransform()));

							// If the recreation of the Traversal Octree was unsuccessful, skip further processing
							if (!RegisteredProxy.TraversalOctree->bValid)
							{
								continue;
							}
							
							RegisteredProxy.PointCloud->Octree.RegisterTraversalOctree(RegisteredProxy.TraversalOctree);
						}

						// If this is an editor component, use its own ViewportClient
						if (TSharedPtr<FViewportClient> Client = Component->GetOwningViewportClient().Pin())
						{
							// If the ViewData cannot be successfully retrieved from the editor viewport, fall back to using main view
							if (!RegisteredProxy.ViewData.ComputeFromEditorViewportClient(Client.Get()))
							{
								RegisteredProxy.ViewData = ViewData;
							}
						}
						// ... otherwise, use the ViewData provided
						else
						{
							RegisteredProxy.ViewData = ViewData;
						}

						// Increase priority, if the viewport has focus
						if (bPrioritizeActiveViewport && RegisteredProxy.ViewData.bHasFocus)
						{
							RegisteredProxy.ViewData.ScreenSizeFactor *= 6;
						}

						// Don't count the skippable proxies
						if (!RegisteredProxy.bSkip)
						{
							TotalPointCount += PointCloud->GetNumPoints();
						}

						// Update render params
						RegisteredProxy.ComponentRenderParams.UpdateFromComponent(Component);

						bValidProxy = true;
					}
				}
			}
		}
		
		// If the SceneProxy has been destroyed, remove it from the list and reiterate
		if(!bValidProxy)
		{
			RegisteredProxies.RemoveAtSwap(i--, EAllowShrinking::No);
		}
	}

	SET_DWORD_STAT(STAT_PointCountTotal, TotalPointCount / 1000);
}

TArray<FLidarPointCloudClippingVolumeParams> FLidarPointCloudLODManager::GetClippingVolumes() const
{
	TArray<FLidarPointCloudClippingVolumeParams> ClippingVolumes;
	TArray<UWorld*> Worlds;

	for (int32 i = 0; i < RegisteredProxies.Num(); ++i)
	{
		if (ULidarPointCloudComponent* Component = RegisteredProxies[i].Component.Get())
		{
			if (!Component->IsOwnedByEditor())
			{
				if (UWorld* World = Component->GetWorld())
				{
					Worlds.AddUnique(World);
				}
			}
		}
	}

	for (UWorld* World : Worlds)
	{
		for (TActorIterator<ALidarClippingVolume> It(World); It; ++It)
		{
			ALidarClippingVolume* Volume = *It;
			if (Volume->bEnabled)
			{
				ClippingVolumes.Emplace(Volume);
			}
		}
	}

	ClippingVolumes.Sort();

	return ClippingVolumes;
}

FLidarPointCloudLODManager& FLidarPointCloudLODManager::Get()
{
	static FLidarPointCloudLODManager Instance;
	return Instance;
}

FLidarPointCloudLODManager::FRegisteredProxy::FRegisteredProxy(TWeakObjectPtr<ULidarPointCloudComponent> Component, TWeakPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper)
	: Component(Component)
	, PointCloud(Component->GetPointCloud())
	, Octree(&PointCloud->Octree)
	, SceneProxyWrapper(SceneProxyWrapper)
	, TraversalOctree(new FLidarPointCloudTraversalOctree(Octree, Component->GetComponentTransform()))
	, LastComponentTransform(Component->GetComponentTransform())
	, bSkip(false)
{
	Octree->RegisterTraversalOctree(TraversalOctree);
}
