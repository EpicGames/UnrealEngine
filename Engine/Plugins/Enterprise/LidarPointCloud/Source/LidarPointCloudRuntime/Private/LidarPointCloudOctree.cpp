// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudOctree.h"
#include "LidarPointCloudOctreeMacros.h"
#include "LidarPointCloud.h"
#include "Meshing/LidarPointCloudMeshing.h"
#include "Misc/ScopeTryLock.h"
#include "Containers/Queue.h"
#include "Async/Async.h"
#include "Misc/FileHelper.h"
#include "Rendering/LidarPointCloudRenderBuffers.h"
#include "UObject/Package.h"

int32 FLidarPointCloudOctree::MaxNodeDepth = (1 << (sizeof(FLidarPointCloudOctreeNode::Depth) * 8)) - 1;
int32 FLidarPointCloudOctree::MaxBucketSize = 200;
int32 FLidarPointCloudOctree::NodeGridResolution = 96;

FArchive& operator<<(FArchive& Ar, FLidarPointCloudPoint_Legacy& P)
{
	Ar << P.Location << P.Color;

	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) > 8)
	{
		uint8 bVisible = P.bVisible;
		Ar << bVisible;
		P.bVisible = bVisible;
	}

	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) > 12)
	{
		uint8 ClassificationID = P.ClassificationID;
		Ar << ClassificationID;
		P.ClassificationID = ClassificationID;
	}

	return Ar;
}

struct FLidarPointCloudBulkData_Legacy : public FBulkData
{
private:
	int32 ElementSize;

public:
	FLidarPointCloudBulkData_Legacy(int32 ElementSize) : ElementSize(ElementSize) {}
	void Serialize(FArchive& Ar, UObject* Owner);
	int32 GetElementCount() const;
};

void FLidarPointCloudBulkData_Legacy::Serialize(FArchive& Ar, UObject* Owner)
{
	const bool bAttemptFileMapping = false;
	FBulkData::Serialize(Ar, Owner, bAttemptFileMapping, ElementSize, EFileRegionType::None);
}

int32 FLidarPointCloudBulkData_Legacy::GetElementCount() const
{
	return GetBulkDataSize() / ElementSize;
}

/** Used for grid allocation calculations */
struct FGridAllocation
{
	/** Index to the point inside of the AllocatedPoints */
	int32 Index;

	/** Index of the child node this point would be placed in */
	int32 ChildNodeLocation;

	/** The calculated distance squared from the center of the given point */
	float DistanceFromCenter;

	FGridAllocation() {}
	FGridAllocation(const int32& Index, const FGridAllocation& GridAllocation)
		: Index(Index)
		, ChildNodeLocation(GridAllocation.ChildNodeLocation)
		, DistanceFromCenter(GridAllocation.DistanceFromCenter)
	{
	}
};

FGridAllocation CalculateGridCellData(const FVector3f& Location, const FVector3f& Center, const FLidarPointCloudOctree::FSharedLODData& LODData)
{
	const FVector3f CenterRelativeLocation = Location - Center;
	const FVector3f OffsetLocation = CenterRelativeLocation + LODData.Extent;
	const FVector3f NormalizedGridLocation = OffsetLocation * LODData.NormalizationMultiplier;

	// Calculate the location on this node's Grid
	const int32 GridX = FMath::Min(FLidarPointCloudOctree::NodeGridResolution - 1, (int32)NormalizedGridLocation.X);
	const int32 GridY = FMath::Min(FLidarPointCloudOctree::NodeGridResolution - 1, (int32)NormalizedGridLocation.Y);
	const int32 GridZ = FMath::Min(FLidarPointCloudOctree::NodeGridResolution - 1, (int32)NormalizedGridLocation.Z);

	FGridAllocation Allocation;
	Allocation.Index = GridX * FLidarPointCloudOctree::NodeGridResolution * FLidarPointCloudOctree::NodeGridResolution + GridY * FLidarPointCloudOctree::NodeGridResolution + GridZ;
	Allocation.DistanceFromCenter = (FVector3f(GridX + 0.5f, GridY + 0.5f, GridZ + 0.5f) * LODData.GridSize3D - OffsetLocation).SizeSquared();
	Allocation.ChildNodeLocation = (CenterRelativeLocation.X > 0 ? 4 : 0) + (CenterRelativeLocation.Y > 0 ? 2 : 0) + (CenterRelativeLocation.Z > 0);

	return Allocation;
}

FORCEINLINE float BrightnessFromColor(const FColor& Color) { return 0.2126 * Color.R + 0.7152 * Color.G + 0.0722 * Color.B; }

bool IsOnBoundsEdge(const FBox& Bounds, const FVector3f& Location)
{
	return (Location.X == Bounds.Min.X) || (Location.X == Bounds.Max.X) || (Location.Y == Bounds.Min.Y) || (Location.Y == Bounds.Max.Y) || (Location.Z == Bounds.Min.Z) || (Location.Z == Bounds.Max.Z);
}

//////////////////////////////////////////////////////////// FSharedLODData

FLidarPointCloudOctree::FSharedLODData::FSharedLODData(const FVector3f& InExtent)
{
	const float UniformExtent = InExtent.GetMax();

	Extent = FVector3f(UniformExtent);
	Radius = UniformExtent * 1.73205081f; // sqrt(3)
	RadiusSq = Radius * Radius;
	Size = UniformExtent * 2;
	GridSize = Size / FLidarPointCloudOctree::NodeGridResolution;
	GridSize3D = FVector3f(GridSize);
	NormalizationMultiplier = FLidarPointCloudOctree::NodeGridResolution / Size;
}

//////////////////////////////////////////////////////////// FLidarPointCloudOctreeNode

FLidarPointCloudOctreeNode::FLidarPointCloudOctreeNode(FLidarPointCloudOctree* Tree, const uint8& Depth, const uint8& LocationInParent, const FVector3f& Center)
	: BulkDataLifetime(0)
	, Depth(Depth)
	, LocationInParent(LocationInParent)
	, Center(Center)
	, Tree(Tree)
	, bVisibilityDirty(false)
	, bInUse(false)
	, bHasSelection(false)
	, NumVisiblePoints(0)
	, NumPoints(0)
	, bHasData(false)
	, DataCache(nullptr)
	, VertexFactory(nullptr)
	, RayTracingGeometry(nullptr)
	, bRenderDataDirty(true)
	, bHasDataPending(false)
	, bCanReleaseData(true)
{
	if (Tree)
	{
		Tree->NodeCount[Depth].Increment();
	}
}

FLidarPointCloudOctreeNode::~FLidarPointCloudOctreeNode()
{
	ReleaseDataCache();

	for (int32 i = 0; i < Children.Num(); i++)
	{
		delete Children[i];
		Children[i] = nullptr;
	}
}

void FLidarPointCloudOctreeNode::UpdateNumVisiblePoints()
{
	if (bVisibilityDirty)
	{
		// Sort points to speed up rendering
		SortVisiblePoints();

		// Recalculate visibility
		NumVisiblePoints = 0;
		FOR_RO(Point, this)
		{
			if (!Point->bVisible)
			{
				break;
			}

			NumVisiblePoints++;
		}

		bVisibilityDirty = false;
	}
}

FLidarPointCloudPoint* FLidarPointCloudOctreeNode::GetData() const
{
	FLidarPointCloudOctreeNode* mutable_this = const_cast<FLidarPointCloudOctreeNode*>(this);

	if (!bHasData && NumPoints > 0)
	{
		Tree->StreamNodeData(mutable_this);
	}

	return mutable_this->Data.GetData();
}

FLidarPointCloudPoint* FLidarPointCloudOctreeNode::GetPersistentData() const
{
	FLidarPointCloudOctreeNode* mutable_this = const_cast<FLidarPointCloudOctreeNode*>(this);
	mutable_this->bCanReleaseData = false;
	return GetData();
}

bool FLidarPointCloudOctreeNode::BuildDataCache(bool bUseStaticBuffers, bool bUseRayTracing)
{	
	// Only include nodes with available data
	if (HasData() && GetNumVisiblePoints())
	{
		// Make sure to release the unnecessary buffer
		if (bUseStaticBuffers)
		{
			if (DataCache.IsValid())
			{
				DataCache->ReleaseResource();
				DataCache.Reset();
				bRenderDataDirty = true;
			}

			if (!VertexFactory.IsValid())
			{
				VertexFactory = MakeShareable(new FLidarPointCloudVertexFactory());
				bRenderDataDirty = true;
			}
		}
		else
		{
			if (VertexFactory.IsValid())
			{
				VertexFactory->ReleaseResource();
				VertexFactory.Reset();
				bRenderDataDirty = true;
			}

			if (!DataCache.IsValid())
			{
				DataCache = MakeShareable(new FLidarPointCloudRenderBuffer());
				bRenderDataDirty = true;
			}
		}

		if(bUseRayTracing)
		{
			if(!RayTracingGeometry.IsValid())
			{
				RayTracingGeometry = MakeShareable(new FLidarPointCloudRayTracingGeometry());
				bRenderDataDirty = true;
			}
		}
		else
		{
			if(RayTracingGeometry.IsValid())
			{
				RayTracingGeometry->ReleaseResource();
				RayTracingGeometry.Reset();
				bRenderDataDirty = true;
			}
		}

		if (bRenderDataDirty)
		{
			if (DataCache.IsValid())
			{
				DataCache->Initialize(GetData(), GetNumVisiblePoints());
			}

			if (VertexFactory.IsValid())
			{
				VertexFactory->Initialize(GetData(), GetNumVisiblePoints());
			}

			if(RayTracingGeometry.IsValid())
			{
				RayTracingGeometry->Initialize(GetNumVisiblePoints());
			}

			bRenderDataDirty = false;
		}

		return true;
	}

	return false;
}

FORCEINLINE FBox FLidarPointCloudOctreeNode::GetBounds() const
{
	return FBox(Center - Tree->SharedData[Depth].Extent, Center + Tree->SharedData[Depth].Extent);
}

FORCEINLINE FSphere FLidarPointCloudOctreeNode::GetSphereBounds() const
{
	return FSphere((FVector)Center, Tree->SharedData[Depth].Radius);
}

FLidarPointCloudOctreeNode* FLidarPointCloudOctreeNode::GetChildNodeAtLocation(const uint8& Location) const
{
	for (FLidarPointCloudOctreeNode* Child : Children)
	{
		if (Child->LocationInParent == Location)
		{
			return Child;
		}
	}

	return nullptr;
}

uint8 FLidarPointCloudOctreeNode::GetChildrenBitmask() const
{
	uint8 Bitmask = 0;

	for (FLidarPointCloudOctreeNode* Child : Children)
	{
		Bitmask |= 1 << Child->LocationInParent;
	}

	return Bitmask;
}

void FLidarPointCloudOctreeNode::InsertPoints(const FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation)
{
	InsertPoints_Internal(Points, Count, DuplicateHandling, Translation);
}

void FLidarPointCloudOctreeNode::InsertPoints_Dynamic(const FLidarPointCloudPoint* Points, const int64& Count, const FVector3f& Translation)
{
	if (Translation.IsNearlyZero())
	{
		Data.Append(Points, Count);
	}
	else
	{
		for (const FLidarPointCloudPoint* PointsPtr = Points, *DataEnd = PointsPtr + Count; PointsPtr != DataEnd; ++PointsPtr)
		{
			Data.Emplace(PointsPtr->Location + Translation, PointsPtr->Color, !!PointsPtr->bVisible, PointsPtr->ClassificationID, PointsPtr->Normal);
		}
	}
}

void FLidarPointCloudOctreeNode::InsertPoints_Static(const FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation)
{
	const FLidarPointCloudOctree::FSharedLODData& LODData = Tree->SharedData[Depth];

	// Local 
	TArray<FLidarPointCloudPoint> PointBuckets[8];
	TMultiMap<int32, FGridAllocation> NewGridAllocationMap, CurrentGridAllocationMap;

	int32 NumPointsAdded = 0;

	const float MaxDistanceForDuplicate = GetDefault<ULidarPointCloudSettings>()->MaxDistanceForDuplicate;

	// Filter the local set of incoming data
	for (int32 Index = 0; Index < Count; ++Index)
	{
		const FVector3f AdjustedLocation = Points[Index].Location + Translation;
		FGridAllocation InGridData = CalculateGridCellData(AdjustedLocation, Center, LODData);

		// Attempt to allocate the point to this node
		if (FGridAllocation* GridCell = NewGridAllocationMap.Find(InGridData.Index))
		{
			bool bStoreInBucket = true;

			if (DuplicateHandling != ELidarPointCloudDuplicateHandling::Ignore && Points[GridCell->Index].Location.Equals(Points[Index].Location, MaxDistanceForDuplicate))
			{
				if (DuplicateHandling == ELidarPointCloudDuplicateHandling::SelectFirst || BrightnessFromColor(Points[Index].Color) <= BrightnessFromColor(Points[GridCell->Index].Color))
				{
					continue;
				}
				else 
				{
					bStoreInBucket = false;
				}				
			}

			if (InGridData.DistanceFromCenter < GridCell->DistanceFromCenter)
			{
				if (bStoreInBucket)
				{
					const FLidarPointCloudPoint& Other = Points[GridCell->Index];
					PointBuckets[GridCell->ChildNodeLocation].Emplace(Other.Location + Translation, Other.Color, !!Other.bVisible, Other.ClassificationID, Other.Normal);
				}
				
				GridCell->Index = Index;
				GridCell->DistanceFromCenter = InGridData.DistanceFromCenter;
			}
			else if(bStoreInBucket)
			{
				const FLidarPointCloudPoint& Other = Points[Index];
				PointBuckets[InGridData.ChildNodeLocation].Emplace(AdjustedLocation, Other.Color, !!Other.bVisible, Other.ClassificationID, Other.Normal);
			}
		}
		else
		{
			NewGridAllocationMap.Add(InGridData.Index, FGridAllocation(Index, InGridData));
		}
	}

	GetPersistentData();

	// Process incoming points
	{
		FScopeLock Lock(&MapLock);

		// Rebuild Current Grid Mapping
		for (int32 i = 0; i < Data.Num(); ++i)
		{
			FGridAllocation InGridData = CalculateGridCellData(Data[i].Location, Center, LODData);

			// Attempt to allocate the point to this node
			if (FGridAllocation* GridCell = CurrentGridAllocationMap.Find(InGridData.Index))
			{
				if (InGridData.DistanceFromCenter < GridCell->DistanceFromCenter)
				{
					PointBuckets[GridCell->ChildNodeLocation].Add(Data[GridCell->Index]);
					Data[GridCell->Index] = Data[i];
					GridCell->DistanceFromCenter = InGridData.DistanceFromCenter;
				}
				else
				{
					PointBuckets[InGridData.ChildNodeLocation].Add(Data[i]);
				}

				Data.RemoveAtSwap(i--, EAllowShrinking::No);
				--NumPointsAdded;
			}
			else
			{
				CurrentGridAllocationMap.Add(InGridData.Index, FGridAllocation(i, InGridData));
			}
		}

		// Compare the incoming data to the currently held set, and replace if necessary
		for (TPair<int32, FGridAllocation>& Element : NewGridAllocationMap)
		{
			const int32& GridIndex = Element.Key;
			const FLidarPointCloudPoint& Point = Points[Element.Value.Index];
			const FVector3f AdjustedLocation = Point.Location + Translation;

			// Attempt to allocate the point to this node
			if (FGridAllocation* GridCell = CurrentGridAllocationMap.Find(GridIndex))
			{
				FLidarPointCloudPoint& AllocatedPoint = Data[GridCell->Index];
				bool bStoreInBucket = true;

				if (DuplicateHandling != ELidarPointCloudDuplicateHandling::Ignore && AllocatedPoint.Location.Equals((FVector3f)AdjustedLocation, MaxDistanceForDuplicate))
				{
					if (DuplicateHandling == ELidarPointCloudDuplicateHandling::SelectFirst || BrightnessFromColor(Point.Color) <= BrightnessFromColor(AllocatedPoint.Color))
					{
						continue;
					}
					else
					{
						bStoreInBucket = false;
					}
				}

				// If the new point's distance from center of node is shorter than the existing point's, replace the point
				if (Element.Value.DistanceFromCenter < GridCell->DistanceFromCenter)
				{
					if (bStoreInBucket)
					{
						PointBuckets[GridCell->ChildNodeLocation].Add(AllocatedPoint);
					}

					AllocatedPoint.Location = (FVector3f)AdjustedLocation;
					AllocatedPoint.Color = Point.Color;
					AllocatedPoint.bVisible = Point.bVisible;
					AllocatedPoint.ClassificationID = Point.ClassificationID;
					AllocatedPoint.Normal = Point.Normal;
					GridCell->DistanceFromCenter = Element.Value.DistanceFromCenter;
				}
				// ... otherwise add it straight to the bucket
				else if (bStoreInBucket)
				{
					PointBuckets[Element.Value.ChildNodeLocation].Emplace(AdjustedLocation, Point.Color, !!Point.bVisible, Point.ClassificationID, Point.Normal);
				}
			}
			else
			{
				CurrentGridAllocationMap.Add(GridIndex, FGridAllocation(Data.Emplace(AdjustedLocation, Point.Color, !!Point.bVisible, Point.ClassificationID, Point.Normal), Element.Value));
				NumPointsAdded++;
			}
		}

		for (uint8 i = 0; i < 8; ++i)
		{
			if (!GetChildNodeAtLocation(i))
			{
				// While the threads are locked, check if any child nodes need creating
				if (Depth < FLidarPointCloudOctree::MaxNodeDepth && PointBuckets[i].Num() > FLidarPointCloudOctree::MaxBucketSize)
				{
					const FVector3f ChildNodeCenter = Center + LODData.Extent * (FVector3f(-0.5f) + FVector3f((i & 4) == 4, (i & 2) == 2, (i & 1) == 1));
					Children.Add(new FLidarPointCloudOctreeNode(Tree,  Depth + 1, i, ChildNodeCenter));

					// The recursive InserPoints call will happen later, after the Lock is released
				}
				// ... otherwise, points can be re-added back as padding
				else
				{
					Data.Append(PointBuckets[i]);
					NumPointsAdded += PointBuckets[i].Num();
					PointBuckets[i].Reset();
				}
			}
		}

		// Shrink the data usage
		Data.Shrink();

		NumPoints = Data.Num();		
		bHasData = true;
		bRenderDataDirty = true;
	}

	AddPointCount(NumPointsAdded);

	// Pass surplus points
	for (uint8 i = 0; i < 8; ++i)
	{
		if (PointBuckets[i].Num() > 0)
		{
			GetChildNodeAtLocation(i)->InsertPoints_Static(PointBuckets[i].GetData(), PointBuckets[i].Num(), DuplicateHandling, FVector3f::ZeroVector);
		}
	}
}

void FLidarPointCloudOctreeNode::InsertPoints(FLidarPointCloudPoint** Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation)
{
	InsertPoints_Internal(Points, Count, DuplicateHandling, Translation);
}

void FLidarPointCloudOctreeNode::InsertPoints_Dynamic(FLidarPointCloudPoint** Points, const int64& Count, const FVector3f& Translation)
{
	if (Translation.IsNearlyZero())
	{
		for (FLidarPointCloudPoint** PointsPtr = Points, **DataEnd = PointsPtr + Count; PointsPtr != DataEnd; ++PointsPtr)
		{
			Data.Add(**PointsPtr);
		}
	}
	else
	{
		for (FLidarPointCloudPoint** PointsPtr = Points, **DataEnd = PointsPtr + Count; PointsPtr != DataEnd; ++PointsPtr)
		{
			const FLidarPointCloudPoint* PointData = *PointsPtr;
			Data.Emplace(PointData->Location + Translation, PointData->Color, !!PointData->bVisible, PointData->ClassificationID, PointData->Normal);
		}
	}
}

void FLidarPointCloudOctreeNode::InsertPoints_Static(FLidarPointCloudPoint** Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation)
{
	const FLidarPointCloudOctree::FSharedLODData& LODData = Tree->SharedData[Depth];

	// Local 
	TArray<FLidarPointCloudPoint> PointBuckets[8];
	TMultiMap<int32, FGridAllocation> NewGridAllocationMap, CurrentGridAllocationMap;

	int32 NumPointsAdded = 0;

	const float MaxDistanceForDuplicate = GetDefault<ULidarPointCloudSettings>()->MaxDistanceForDuplicate;

	// Filter the local set of incoming data
	for (int32 Index = 0; Index < Count; Index++)
	{
		const FVector3f AdjustedLocation = Points[Index]->Location + Translation;
		FGridAllocation InGridData = CalculateGridCellData(AdjustedLocation, Center, LODData);

		// Attempt to allocate the point to this node
		if (FGridAllocation* GridCell = NewGridAllocationMap.Find(InGridData.Index))
		{
			bool bStoreInBucket = true;

			if (DuplicateHandling != ELidarPointCloudDuplicateHandling::Ignore && Points[GridCell->Index]->Location.Equals(Points[Index]->Location, MaxDistanceForDuplicate))
			{
				if (DuplicateHandling == ELidarPointCloudDuplicateHandling::SelectFirst || BrightnessFromColor(Points[Index]->Color) <= BrightnessFromColor(Points[GridCell->Index]->Color))
				{
					continue;
				}
				else
				{
					bStoreInBucket = false;
				}
			}

			if (InGridData.DistanceFromCenter < GridCell->DistanceFromCenter)
			{
				if (bStoreInBucket)
				{
					const FLidarPointCloudPoint& Other = *Points[GridCell->Index];
					PointBuckets[GridCell->ChildNodeLocation].Emplace(Other.Location + Translation, Other.Color, !!Other.bVisible, Other.ClassificationID, Other.Normal);
				}

				GridCell->Index = Index;
				GridCell->DistanceFromCenter = InGridData.DistanceFromCenter;
			}
			else if (bStoreInBucket)
			{
				const FLidarPointCloudPoint& Other = *Points[Index];
				PointBuckets[InGridData.ChildNodeLocation].Emplace(AdjustedLocation, Other.Color, !!Other.bVisible, Other.ClassificationID, Other.Normal);
			}
		}
		else
		{
			NewGridAllocationMap.Add(InGridData.Index, FGridAllocation(Index, InGridData));
		}
	}

	GetPersistentData();

	// Process incoming points
	{
		FScopeLock Lock(&MapLock);

		// Rebuild Current Grid Mapping
		for (int32 i = 0; i < Data.Num(); ++i)
		{
			FGridAllocation InGridData = CalculateGridCellData(Data[i].Location, Center, LODData);

			// Attempt to allocate the point to this node
			if (FGridAllocation* GridCell = CurrentGridAllocationMap.Find(InGridData.Index))
			{
				if (InGridData.DistanceFromCenter < GridCell->DistanceFromCenter)
				{
					PointBuckets[GridCell->ChildNodeLocation].Add(Data[GridCell->Index]);
					Data[GridCell->Index] = Data[i];
					GridCell->DistanceFromCenter = InGridData.DistanceFromCenter;
				}
				else
				{
					PointBuckets[InGridData.ChildNodeLocation].Add(Data[i]);
				}

				Data.RemoveAtSwap(i--, EAllowShrinking::No);
				--NumPointsAdded;
			}
			else
			{
				CurrentGridAllocationMap.Add(InGridData.Index, FGridAllocation(i, InGridData));
			}
		}

		// Compare the incoming data to the currently held set, and replace if necessary
		for (TPair<int32, FGridAllocation>& Element : NewGridAllocationMap)
		{
			const int32& GridIndex = Element.Key;
			const FLidarPointCloudPoint& Point = *Points[Element.Value.Index];
			FVector3f AdjustedLocation = Point.Location + Translation;

			// Attempt to allocate the point to this node
			if (FGridAllocation* GridCell = CurrentGridAllocationMap.Find(GridIndex))
			{
				FLidarPointCloudPoint& AllocatedPoint = Data[GridCell->Index];
				bool bStoreInBucket = true;

				if (DuplicateHandling != ELidarPointCloudDuplicateHandling::Ignore && AllocatedPoint.Location.Equals((FVector3f)AdjustedLocation, MaxDistanceForDuplicate))
				{
					if (DuplicateHandling == ELidarPointCloudDuplicateHandling::SelectFirst || BrightnessFromColor(Point.Color) <= BrightnessFromColor(AllocatedPoint.Color))
					{
						continue;
					}
					else
					{
						bStoreInBucket = false;
					}
				}

				// If the new point's distance from center of node is shorter than the existing point's, replace the point
				if (Element.Value.DistanceFromCenter < GridCell->DistanceFromCenter)
				{
					if (bStoreInBucket)
					{
						PointBuckets[GridCell->ChildNodeLocation].Add(AllocatedPoint);
					}

					AllocatedPoint.Location = (FVector3f)AdjustedLocation;
					AllocatedPoint.Color = Point.Color;
					AllocatedPoint.bVisible = Point.bVisible;
					AllocatedPoint.ClassificationID = Point.ClassificationID;
					AllocatedPoint.Normal = Point.Normal;
					GridCell->DistanceFromCenter = Element.Value.DistanceFromCenter;
				}
				// ... otherwise add it straight to the bucket
				else if (bStoreInBucket)
				{
					PointBuckets[Element.Value.ChildNodeLocation].Emplace(AdjustedLocation, Point.Color, !!Point.bVisible, Point.ClassificationID, Point.Normal);
				}
			}
			else
			{
				CurrentGridAllocationMap.Add(GridIndex, FGridAllocation(Data.Emplace(AdjustedLocation, Point.Color, !!Point.bVisible, Point.ClassificationID, Point.Normal), Element.Value));
				NumPointsAdded++;
			}
		}

		for (uint8 i = 0; i < 8; i++)
		{
			if (!GetChildNodeAtLocation(i))
			{
				// While the threads are locked, check if any child nodes need creating
				if (Depth < FLidarPointCloudOctree::MaxNodeDepth && PointBuckets[i].Num() > FLidarPointCloudOctree::MaxBucketSize)
				{
					const FVector3f ChildNodeCenter = Center + LODData.Extent * (FVector3f(-0.5f) + FVector3f((i & 4) == 4, (i & 2) == 2, (i & 1) == 1));
					Children.Add(new FLidarPointCloudOctreeNode(Tree,  Depth + 1, i, ChildNodeCenter));

					// The recursive InserPoints call will happen later, after the Lock is released
				}
				// ... otherwise, points can be re-added back as padding
				else
				{
					Data.Append(PointBuckets[i]);
					NumPointsAdded += PointBuckets[i].Num();
					PointBuckets[i].Reset();
				}
			}
		}

		// Shrink the data usage
		Data.Shrink();

		NumPoints = Data.Num();
		bHasData = true;
		bRenderDataDirty = true;
	}

	AddPointCount(NumPointsAdded);

	// Pass surplus points
	for (uint8 i = 0; i < 8; i++)
	{
		if (PointBuckets[i].Num() > 0)
		{
			GetChildNodeAtLocation(i)->InsertPoints_Static(PointBuckets[i].GetData(), PointBuckets[i].Num(), DuplicateHandling, FVector3f::ZeroVector);
		}
	}
}

template <typename T>
void FLidarPointCloudOctreeNode::InsertPoints_Internal(T Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation)
{
	if (Tree->IsOptimizedForDynamicData())
	{
		Data.Reserve(NumPoints + Count);

		InsertPoints_Dynamic(Points, Count, Translation);

		NumPoints = Data.Num();
		bCanReleaseData = false;
		bRenderDataDirty = true;
		bHasData = true;

		AddPointCount(Count);
	}
	else
	{
		InsertPoints_Static(Points, Count, DuplicateHandling, Translation);
	}
}

void FLidarPointCloudOctreeNode::Empty(bool bRecursive)
{
	if (bHasData)
	{
		bHasData = false;
		bInUse = false;
		NumPoints = 0;
		NumVisiblePoints = 0;
		Data.Empty();
	}

	bRenderDataDirty = true;

	if (bRecursive)
	{
		for (int32 i = 0; i < Children.Num(); i++)
		{
			Children[i]->Empty(true);
		}
	}
}

uint32 FLidarPointCloudOctreeNode::GetMaxDepth() const
{
	uint32 MaxDepth = Depth;

	for (FLidarPointCloudOctreeNode* Child : Children)
	{
		MaxDepth = FMath::Max(MaxDepth, Child->GetMaxDepth());
	}

	return MaxDepth;
}

int64 FLidarPointCloudOctreeNode::GetAllocatedSize(bool bRecursive, bool bIncludeBulkData) const
{
	int64 Size = sizeof(FLidarPointCloudOctreeNode);

	Size += Children.GetAllocatedSize();

	if (bIncludeBulkData)
	{
		Size += sizeof(FLidarPointCloudPoint) * GetNumPoints();
	}

	if (bRecursive)
	{
		for (FLidarPointCloudOctreeNode* Child : Children)
		{
			Size += Child->GetAllocatedSize(true, bIncludeBulkData);
		}
	}

	return Size;
}

void FLidarPointCloudOctreeNode::ReleaseData(bool bForce)
{
	// Check if the data can be released
	if ((bCanReleaseData
#if WITH_EDITOR
		&& !bHasSelection
#endif
		) || bForce)
	{
		bHasDataPending = false;
		bCanReleaseData = true;

		if (bHasData)
		{
			bHasData = false;
			bInUse = false;
			Data.Empty();
		}
	}

	ReleaseDataCache();
}

void FLidarPointCloudOctreeNode::ReleaseDataCache()
{
	if (VertexFactory.IsValid() || DataCache.IsValid())
	{
		ENQUEUE_RENDER_COMMAND(LidarPointCloudOctreeNode_ReleaseDataCache)([
			DataCache = DataCache, VertexFactory = VertexFactory, RayTracingGeometry = RayTracingGeometry](FRHICommandListImmediate& RHICmdList)
		{
			if (DataCache.IsValid())
			{
				DataCache->ReleaseResource();
			}

			if (VertexFactory.IsValid())
			{
				VertexFactory->ReleaseResource();
			}
			
			if(RayTracingGeometry.IsValid())
			{
				RayTracingGeometry->ReleaseResource();
			}
		});

		DataCache.Reset();
		VertexFactory.Reset();
		RayTracingGeometry.Reset();
	}
}

void FLidarPointCloudOctreeNode::AddPointCount(int32 PointCount)
{
	Tree->PointCount[Depth].Add(PointCount);
	NumVisiblePoints += PointCount;
}

void FLidarPointCloudOctreeNode::SortVisiblePoints()
{
	TArrayView<FLidarPointCloudPoint> Points(GetData(), Data.Num());
	Algo::Sort(Points, [](const FLidarPointCloudPoint& A, const FLidarPointCloudPoint& B)
	{
		return A.bVisible > B.bVisible;
	});
	bCanReleaseData = false;
	bRenderDataDirty = true;
}

//////////////////////////////////////////////////////////// FLidarPointCloudOctree

FLidarPointCloudOctree::FLidarPointCloudOctree(ULidarPointCloud* Owner)
	: Root(new FLidarPointCloudOctreeNode(nullptr, 0))
	, Owner(Owner)
#if WITH_EDITOR
	, SavingBulkData(this)
#endif
	, bStreamingBusy(false)
	, bIsFullyLoaded(false)
{
	PointCount.AddDefaulted(MaxNodeDepth + 1);
	NodeCount.AddDefaulted(MaxNodeDepth + 1);
	SharedData.AddDefaulted(MaxNodeDepth + 1);

	// Account for the Root
	NodeCount[0].Increment();

	Root->Tree = this;
}

FLidarPointCloudOctree::~FLidarPointCloudOctree()
{
	Empty(true);
}

int32 FLidarPointCloudOctree::GetNumLODs() const
{
	int32 LODCount = 0;

	for (; LODCount < NodeCount.Num(); LODCount++)
	{
		if (NodeCount[LODCount].GetValue() == 0)
		{
			break;
		}
	}

	return LODCount;
}

void FLidarPointCloudOctree::RefreshBounds()
{
	FBox Bounds(EForceInit::ForceInit);

	// Calculate the current bounds
	ITERATE_NODES_CONST({ FOR_RO(Point, CurrentNode) { Bounds += (FVector)Point->Location; } }, true);

	Extent = (FVector3f)Bounds.GetExtent();
	const FVector3f Offset = (FVector3f)Bounds.GetCenter();

	if (!Offset.IsNearlyZero(0.1f))
	{
		Owner->LocationOffset += (FVector)Offset;
		Owner->OriginalCoordinates += (FVector)Offset;

		// Shift the points back to the relative position
		ITERATE_NODES(
		{
			CurrentNode->Center -= Offset;

			FOR(Point, CurrentNode)
			{
				Point->Location -= Offset;
			}
		}, true);
	}
}

int64 FLidarPointCloudOctree::GetNumPoints() const
{
	int64 TotalPointCount = 0;

	for (int32 i = 0; i < PointCount.Num(); i++)
	{
		int64 NumPoints = PointCount[i].GetValue();

		if (NumPoints > 0)
		{
			TotalPointCount += NumPoints;
		}
		else
		{
			break;
		}
	}

	return TotalPointCount;
}

int64 FLidarPointCloudOctree::GetNumVisiblePoints() const
{
	int64 NumVisiblePoints = 0;

	ITERATE_NODES_CONST({ NumVisiblePoints += CurrentNode->GetNumVisiblePoints(); }, true);

	return NumVisiblePoints;
}

int32 FLidarPointCloudOctree::GetNumNodes() const
{
	int32 TotalNodeCount = 0;

	for (int32 i = 0; i < NodeCount.Num(); i++)
	{
		int32 NumNodes = NodeCount[i].GetValue();
		if (NumNodes > 0)
		{
			TotalNodeCount += NumNodes;
		}
		else
		{
			break;
		}
	}

	return TotalNodeCount;
}

int64 FLidarPointCloudOctree::GetAllocatedSize() const
{
	if (PreviousPointCount != GetNumPoints() || PreviousNodeCount != GetNumNodes())
	{
		FLidarPointCloudOctree* mutable_this = const_cast<FLidarPointCloudOctree*>(this);
		mutable_this->RefreshAllocatedSize();
	}

	return PreviousAllocatedSize;
}

int64 FLidarPointCloudOctree::GetAllocatedStructureSize() const
{
	if (PreviousPointCount != GetNumPoints() || PreviousNodeCount != GetNumNodes())
	{
		FLidarPointCloudOctree* mutable_this = const_cast<FLidarPointCloudOctree*>(this);
		mutable_this->RefreshAllocatedSize();
	}

	return PreviousAllocatedStructureSize;
}

float FLidarPointCloudOctree::GetEstimatedPointSpacing() const
{
	float Spacing = 0;
	const int64 TotalPointCount = GetNumPoints();

	for (int32 i = 0; i < PointCount.Num(); ++i)
	{
		Spacing += SharedData[i].GridSize * PointCount[i].GetValue() / TotalPointCount;
	}

	return Spacing;
}

void FLidarPointCloudOctree::BuildCollision(const float& Accuracy, const bool& bVisibleOnly)
{
	LidarPointCloudMeshing::BuildCollisionMesh(this, Accuracy, &CollisionMesh);
}

void FLidarPointCloudOctree::RemoveCollision()
{
	FScopeLock Lock(&DataLock);

	CollisionMesh.~FTriMeshCollisionData();
	new (&CollisionMesh) FTriMeshCollisionData();
}

void FLidarPointCloudOctree::BuildStaticMeshBuffers(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform)
{
	if(CellSize == 0)
	{
		CellSize = GetEstimatedPointSpacing() * 1.1f;
	}
	
	LidarPointCloudMeshing::BuildStaticMeshBuffers(this, CellSize, false, OutMeshBuffers, Transform);
}

void FLidarPointCloudOctree::GetPoints(TArray<FLidarPointCloudPoint*>& SelectedPoints, int64 StartIndex /*= 0*/, int64 Count /*= -1*/)
{
	GetPoints_Internal(SelectedPoints, StartIndex, Count);
}

void FLidarPointCloudOctree::GetPoints(TArray64<FLidarPointCloudPoint*>& SelectedPoints, int64 StartIndex /*= 0*/, int64 Count /*= -1*/)
{
	GetPoints_Internal(SelectedPoints, StartIndex, Count);
}

template<typename T>
void FLidarPointCloudOctree::GetPoints_Internal(TArray<FLidarPointCloudPoint*, T>& Points, int64 StartIndex, int64 Count)
{
	check(StartIndex >= 0 && StartIndex < GetNumPoints());

	if (Count < 0)
	{
		Count = GetNumPoints();
	}

	Count = FMath::Min(Count, GetNumPoints() - StartIndex);

	Points.Empty(Count);

	FScopeLock Lock(&DataLock);

	ITERATE_NODES(
	{
		// If no data is required, quit
		if (Count == 0)
		{
			return;
		}

		// Should this node's points be injected?
		if (StartIndex < CurrentNode->GetNumPoints())
		{
			// If the index is at 0 and the requested count is at least the size of this node, append whole arrays
			if (StartIndex == 0 && Count >= CurrentNode->GetNumPoints())
			{
				// Expand the array
				int64 Offset = Points.Num();
				Points.AddUninitialized(CurrentNode->GetNumPoints());
				FLidarPointCloudPoint** Dest = Points.GetData() + Offset;

				// Assign pointers to the data
				FOR(Point, CurrentNode)
				{
					*Dest++ = Point;
				}

				Count -= CurrentNode->GetNumPoints();
			}
			// ... otherwise iterate over needed data
			else
			{
				int64 NumPointsToCopy = FMath::Max(0LL, FMath::Min((int64)CurrentNode->GetNumPoints(), Count + StartIndex) - StartIndex);
				if (NumPointsToCopy > 0)
				{
					// Expand the array
					int64 Offset = Points.Num();
					Points.AddUninitialized(NumPointsToCopy);

					// Setup pointers
					FLidarPointCloudPoint* Src = CurrentNode->GetData() + StartIndex;
					FLidarPointCloudPoint** Dest = Points.GetData() + Offset;

					// Assign source pointers to the destination
					for (int64 i = 0; i < NumPointsToCopy; i++)
					{
						*Dest++ = Src++;
					}

					StartIndex = FMath::Max(0LL, StartIndex - NumPointsToCopy);
					Count -= NumPointsToCopy;
				}
			}
		}
		// ... or skipped completely?
		else
		{
			StartIndex -= CurrentNode->GetNumPoints();
		}
	}, true);
}

template <typename T>
void FLidarPointCloudOctree::GetPointsInSphere_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
{
	SelectedPoints.Reset();
	PROCESS_IN_SPHERE({ SelectedPoints.Add(Point); });
}

template <typename T>
void FLidarPointCloudOctree::GetPointsInBox_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
{
	SelectedPoints.Reset(); 
	PROCESS_IN_BOX({ SelectedPoints.Add(Point); });
}

template <typename T>
void FLidarPointCloudOctree::GetPointsInBox_Internal(TArray<const FLidarPointCloudPoint*, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly) const
{
	SelectedPoints.Reset();
	PROCESS_IN_BOX_CONST({ SelectedPoints.Add(Point); });
}

template <typename T>
void FLidarPointCloudOctree::GetPointsInConvexVolume_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly)
{
	SelectedPoints.Reset();
	PROCESS_IN_CONVEX_VOLUME({ SelectedPoints.Add(Point); });
}

template <typename T>
void FLidarPointCloudOctree::GetPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& Points, const FTransform* LocalToWorld, int64 StartIndex, int64 Count) const
{
	// If empty, abort
	if (GetNumPoints() == 0)
	{
		return;
	}

	// Make sure to operate on correct range
	check(StartIndex >= 0 && StartIndex < GetNumPoints());

	if (Count < 0)
	{
		Count = GetNumPoints();
	}

	Count = FMath::Min(Count, GetNumPoints() - StartIndex);

	// TArray only supports int32 ax max number of elements!
	check(Count <= INT32_MAX);

	Points.Empty(Count);

	FScopeLock Lock(&DataLock);

	if (LocalToWorld)
	{
		const FTransform3f Transform = (FTransform3f)*LocalToWorld;
		ITERATE_NODES_CONST({
			// If no data is required, quit
			if (Count == 0)
			{
				return;
			}

			// Should this node's points be injected?
			if (StartIndex < CurrentNode->GetNumPoints())
			{
				const int32 NumPointsToCopy = FMath::Max(0LL, FMath::Min((int64)CurrentNode->GetNumPoints(), Count + StartIndex) - StartIndex);
				if (NumPointsToCopy > 0)
				{
					for (FLidarPointCloudPoint* Point = CurrentNode->GetData() + StartIndex, *DataEnd = Point + StartIndex + NumPointsToCopy; Point != DataEnd; ++Point)
					{
						Points.Add(Point->Transform(Transform));
					}

					StartIndex = FMath::Max(0LL, StartIndex - NumPointsToCopy);
					Count -= NumPointsToCopy;
				}
			}
			// ... or skipped completely?
			else
			{
				StartIndex -= CurrentNode->GetNumPoints();
			}
		}, true);
	}
	else
	{
		ITERATE_NODES_CONST({
			// If no data is required, quit
			if (Count == 0)
			{
				return;
			}

			// Should this node's points be injected?
			if (StartIndex < CurrentNode->GetNumPoints())
			{
				// If the index is at 0 and the requested count is at least the size of this node, append whole arrays
				if (StartIndex == 0 && Count >= CurrentNode->GetNumPoints())
				{
					Points.Append(CurrentNode->GetData(), CurrentNode->GetNumPoints());
					Count -= CurrentNode->GetNumPoints();
				}
				// ... otherwise iterate over needed data
				else
				{
					int32 NumPointsToCopy = FMath::Max(0LL, FMath::Min((int64)CurrentNode->GetNumPoints(), Count + StartIndex) - StartIndex);
					if (NumPointsToCopy > 0)
					{
						int32 SrcOffset = StartIndex;
						int32 DestOffset = Points.Num();

						// Expand the array
						Points.AddUninitialized(NumPointsToCopy);

						// Copy contents
						FMemory::Memcpy(Points.GetData() + DestOffset, CurrentNode->GetData() + SrcOffset, NumPointsToCopy * sizeof(FLidarPointCloudPoint));

						StartIndex = FMath::Max(0LL, StartIndex - NumPointsToCopy);
						Count -= NumPointsToCopy;
					}
				}
			}
			// ... or skipped completely?
			else
			{
				StartIndex -= CurrentNode->GetNumPoints();
			}
		}, true);
	}
}

void FLidarPointCloudOctree::GetPointsInSphere(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
{
	GetPointsInSphere_Internal(SelectedPoints, Sphere, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInSphere(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
{
	GetPointsInSphere_Internal(SelectedPoints, Sphere, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInBox(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
{
	GetPointsInBox_Internal(SelectedPoints, Box, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInBox(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
{
	GetPointsInBox_Internal(SelectedPoints, Box, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInBox(TArray64<const FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly) const
{
	GetPointsInBox_Internal(SelectedPoints, Box, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInBox(TArray<const FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly) const
{
	GetPointsInBox_Internal(SelectedPoints, Box, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInConvexVolume(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, ConvexVolume, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInConvexVolume(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, ConvexVolume, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInFrustum(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, Frustum, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsInFrustum(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly)
{
	GetPointsInConvexVolume_Internal(SelectedPoints, Frustum, bVisibleOnly);
}

void FLidarPointCloudOctree::GetPointsAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FTransform* LocalToWorld, int64 StartIndex /*= 0*/, int64 Count /*= -1*/) const
{
	GetPointsAsCopies_Internal(SelectedPoints, LocalToWorld, StartIndex, Count);
}

void FLidarPointCloudOctree::GetPointsAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FTransform* LocalToWorld, int64 StartIndex /*= 0*/, int64 Count /*= -1*/) const
{
	GetPointsAsCopies_Internal(SelectedPoints, LocalToWorld, StartIndex, Count);
}

void FLidarPointCloudOctree::GetPointsAsCopiesInBatches(TFunction<void(TSharedPtr<TArray64<FLidarPointCloudPoint>>)> Action, const int64& BatchSize, const bool& bVisibleOnly)
{
	TSharedPtr<TArray64<FLidarPointCloudPoint>> Points(new TArray64<FLidarPointCloudPoint>());
	Points->Reserve(BatchSize);

	TQueue<FLidarPointCloudOctreeNode*> Nodes;
	FLidarPointCloudOctreeNode* CurrentNode = nullptr;
	Nodes.Enqueue(Root);
	while (Nodes.Dequeue(CurrentNode))
	{
		FOR_RO(Point, CurrentNode)
		{
			if (Points->Add(*Point) == BatchSize - 1)
			{
				Action(Points);
				Points = TSharedPtr<TArray64<FLidarPointCloudPoint>>(new TArray64<FLidarPointCloudPoint>());
				Points->Reserve(BatchSize);
			}
		}

		for (FLidarPointCloudOctreeNode* Child : CurrentNode->Children)
		{
			Nodes.Enqueue(Child);
		}

		CurrentNode->ReleaseData(false);
	}

	if (Points->Num() > 0)
	{
		Action(Points);
	}
}

template <typename T>
void FLidarPointCloudOctree::GetPointsInSphereAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const FTransform* LocalToWorld) const
{
	SelectedPoints.Reset();
	if (LocalToWorld)
	{
		const FTransform3f Transform = (FTransform3f)*LocalToWorld;
		PROCESS_IN_SPHERE_CONST({ SelectedPoints.Add(Point->Transform(Transform)); });
	}
	else
	{
		PROCESS_IN_SPHERE_CONST({ SelectedPoints.Add(*Point); });
	}
}

template <typename T>
void FLidarPointCloudOctree::GetPointsInBoxAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const FTransform* LocalToWorld) const
{
	SelectedPoints.Reset();
	if (LocalToWorld)
	{
		const FTransform3f Transform = (FTransform3f)*LocalToWorld;
		PROCESS_IN_BOX_CONST({ SelectedPoints.Add(Point->Transform(Transform)); });
	}
	else
	{
		PROCESS_IN_BOX_CONST({ SelectedPoints.Add(*Point); });
	}
}

void FLidarPointCloudOctree::GetPointsInSphereAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const FTransform* LocalToWorld) const
{
	GetPointsInSphereAsCopies_Internal(SelectedPoints, Sphere, bVisibleOnly, LocalToWorld);
}

void FLidarPointCloudOctree::GetPointsInSphereAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const FTransform* LocalToWorld) const
{
	GetPointsInSphereAsCopies_Internal(SelectedPoints, Sphere, bVisibleOnly, LocalToWorld);
}

void FLidarPointCloudOctree::GetPointsInBoxAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const FTransform* LocalToWorld) const
{
	GetPointsInBoxAsCopies_Internal(SelectedPoints, Box, bVisibleOnly, LocalToWorld);
}

void FLidarPointCloudOctree::GetPointsInBoxAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const FTransform* LocalToWorld) const
{
	GetPointsInBoxAsCopies_Internal(SelectedPoints, Box, bVisibleOnly, LocalToWorld);
}

FLidarPointCloudPoint* FLidarPointCloudOctree::RaycastSingle(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly)
{
	PROCESS_BY_RAY({ return Point; });
	return nullptr;
}

bool FLidarPointCloudOctree::RaycastMulti(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly, TArray<FLidarPointCloudPoint*>& OutHits)
{
	OutHits.Reset();
	PROCESS_BY_RAY({ OutHits.Add(Point); });
	return OutHits.Num() > 0;
}

bool FLidarPointCloudOctree::RaycastMulti(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly, const FTransform* LocalToWorld, TArray<FLidarPointCloudPoint>& OutHits)
{
	OutHits.Reset();
	if (LocalToWorld)
	{
		const FTransform3f Transform = (FTransform3f)*LocalToWorld;
		PROCESS_BY_RAY_CONST({ OutHits.Add(Point->Transform(Transform)); });
	}
	else
	{
		PROCESS_BY_RAY_CONST({ OutHits.Add(*Point); });
	}
	return OutHits.Num() > 0;
}

bool FLidarPointCloudOctree::HasPointsInSphere(const FSphere& Sphere, const bool& bVisibleOnly) const
{
	PROCESS_IN_SPHERE_CONST({ return true; });
	return false;
}

bool FLidarPointCloudOctree::HasPointsInBox(const FBox& Box, const bool& bVisibleOnly) const
{
	PROCESS_IN_BOX_CONST({ return true; });
	return false;
}

bool FLidarPointCloudOctree::HasPointsByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly) const
{
	PROCESS_BY_RAY_CONST({ return true; });
	return false;
}

void FLidarPointCloudOctree::SetVisibilityOfPointsInSphere(const bool& bNewVisibility, const FSphere& Sphere)
{
	// Build a box to quickly filter out the points - (IsInsideOrOn vs comparing DistSquared)
	const FBox Box(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));
	const float RadiusSq = Sphere.W * Sphere.W;

	ITERATE_NODES({
		// Skip node if it already has all points set to the required visibility state
		bool bSkipNode = (CurrentNode->NumVisiblePoints == CurrentNode->GetNumPoints() && bNewVisibility) || (CurrentNode->NumVisiblePoints == 0 && !bNewVisibility);
		if (!bSkipNode)
		{
			CurrentNode->NumVisiblePoints = 0;

			// If node fully inside the radius - do not check individual points
			if (CurrentNode->GetSphereBounds().IsInside(Sphere))
			{
				FOR(Point, CurrentNode)
				{
					Point->bVisible = bNewVisibility;
				}

				if (bNewVisibility)
				{
					CurrentNode->NumVisiblePoints = CurrentNode->GetNumPoints();
				}
			}
			else
			{
				FOR(Point, CurrentNode)
				{
					if (Point->bVisible != bNewVisibility && POINT_IN_SPHERE)
					{
						Point->bVisible = bNewVisibility;
					}

					if (Point->bVisible)
					{
						++CurrentNode->NumVisiblePoints;
					}
				}
			}

			CurrentNode->bVisibilityDirty = false;

			// Sort points to speed up rendering
			CurrentNode->SortVisiblePoints();
			CurrentNode->bRenderDataDirty = true;
		}
	}, NODE_IN_BOX);
}

void FLidarPointCloudOctree::SetVisibilityOfPointsInBox(const bool& bNewVisibility, const FBox& Box)
{
	ITERATE_NODES({
		// Skip node if it already has all points set to the required visibility state
		bool bSkipNode = (CurrentNode->NumVisiblePoints == CurrentNode->GetNumPoints() && bNewVisibility) || (CurrentNode->NumVisiblePoints == 0 && !bNewVisibility);
		if (!bSkipNode)
		{
			CurrentNode->NumVisiblePoints = 0;

			// If node fully inside the radius - do not check individual points
			if (Box.IsInsideOrOn((FVector)(CurrentNode->Center - SharedData[CurrentNode->Depth].Extent)) && Box.IsInsideOrOn((FVector)(CurrentNode->Center + SharedData[CurrentNode->Depth].Extent)))
			{
				FOR(Point, CurrentNode)
				{
					Point->bVisible = bNewVisibility;
				}

				if (bNewVisibility)
				{
					CurrentNode->NumVisiblePoints = CurrentNode->GetNumPoints();
				}
			}
			else
			{
				FOR(Point, CurrentNode)
				{
					if (Point->bVisible != bNewVisibility && POINT_IN_BOX)
					{
						Point->bVisible = bNewVisibility;
					}

					if (Point->bVisible)
					{
						++CurrentNode->NumVisiblePoints;
					}
				}
			}

			CurrentNode->bVisibilityDirty = false;

			// Sort points to speed up rendering
			CurrentNode->SortVisiblePoints();
			CurrentNode->bRenderDataDirty = true;
		}
	}, NODE_IN_BOX);
}

void FLidarPointCloudOctree::SetVisibilityOfFirstPointByRay(const bool& bNewVisibility, const FLidarPointCloudRay& Ray, const float& Radius)
{
	bool bPointProcessed = false;
	const float RadiusSq = Radius * Radius;
	ITERATE_NODES({
		// Skip node if it already has all points set to the required visibility state
		bool bSkipNode = (CurrentNode->NumVisiblePoints == CurrentNode->GetNumPoints() && bNewVisibility) || (CurrentNode->NumVisiblePoints == 0 && !bNewVisibility);
		if (!bSkipNode && Ray.Intersects(CurrentNode->GetBounds()))
		{
			CurrentNode->NumVisiblePoints = 0;

			FOR(Point, CurrentNode)
			{
				if (Point->bVisible != bNewVisibility && POINT_BY_RAY)
				{
					Point->bVisible = bNewVisibility;
					bPointProcessed = true;
				}

				if (Point->bVisible)
				{
					++CurrentNode->NumVisiblePoints;
				}

				if (bPointProcessed)
				{
					break;
				}
			}

			CurrentNode->bVisibilityDirty = false;

			// Sort points to speed up rendering
			CurrentNode->SortVisiblePoints();
			CurrentNode->bRenderDataDirty = true;

			if (bPointProcessed)
			{
				return;
			}

			for (FLidarPointCloudOctreeNode*& Child : CurrentNode->Children)
			{
				Nodes.Enqueue(Child);
			}
		}
	}, false);
}

void FLidarPointCloudOctree::SetVisibilityOfPointsByRay(const bool& bNewVisibility, const FLidarPointCloudRay& Ray, const float& Radius)
{
	const float RadiusSq = Radius * Radius;
	ITERATE_NODES({
		// Skip node if it already has all points set to the required visibility state
		bool bSkipNode = (CurrentNode->NumVisiblePoints == CurrentNode->GetNumPoints() && bNewVisibility) || (CurrentNode->NumVisiblePoints == 0 && !bNewVisibility);
		if (!bSkipNode && Ray.Intersects(CurrentNode->GetBounds()))
		{
			CurrentNode->NumVisiblePoints = 0;

			FOR(Point, CurrentNode)
			{
				if (Point->bVisible != bNewVisibility && POINT_BY_RAY)
				{
					Point->bVisible = bNewVisibility;
				}

				if (Point->bVisible)
				{
					++CurrentNode->NumVisiblePoints;
				}
			}

			CurrentNode->bVisibilityDirty = false;

			// Sort points to speed up rendering
			CurrentNode->SortVisiblePoints();
			CurrentNode->bRenderDataDirty = true;

			for (FLidarPointCloudOctreeNode*& Child : CurrentNode->Children)
			{
				Nodes.Enqueue(Child);
			}
		}
	}, false);
}

void FLidarPointCloudOctree::HideAll()
{
	ITERATE_NODES({
		if (CurrentNode->NumVisiblePoints > 0)
		{
			FOR(Point, CurrentNode)
			{
				Point->bVisible = false;
			}

			CurrentNode->NumVisiblePoints = 0;
			CurrentNode->bVisibilityDirty = false;
			CurrentNode->bRenderDataDirty = true;
		}
	}, true);
}

void FLidarPointCloudOctree::UnhideAll()
{
	ITERATE_NODES({
		if (CurrentNode->NumVisiblePoints != CurrentNode->GetNumPoints())
		{
			FOR(Point, CurrentNode)
			{
				Point->bVisible = true;
			}

			CurrentNode->NumVisiblePoints = CurrentNode->GetNumPoints();
			CurrentNode->bVisibilityDirty = false;
			CurrentNode->bRenderDataDirty = true;
		}
	}, true);
}

void FLidarPointCloudOctree::ExecuteActionOnAllPoints(TFunction<void(FLidarPointCloudPoint*)> Action, const bool& bVisibleOnly)
{
	PROCESS_ALL_EX({ Action(Point); }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ExecuteActionOnPointsInSphere(TFunction<void(FLidarPointCloudPoint*)> Action, const FSphere& Sphere, const bool& bVisibleOnly)
{
	PROCESS_IN_SPHERE_EX({ Action(Point); }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ExecuteActionOnPointsInBox(TFunction<void(FLidarPointCloudPoint*)> Action, const FBox& Box, const bool& bVisibleOnly)
{
	PROCESS_IN_BOX_EX({ Action(Point); }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ExecuteActionOnFirstPointByRay(TFunction<void(FLidarPointCloudPoint*)> Action, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly)
{
	PROCESS_BY_RAY({
		Action(Point);
		CurrentNode->bRenderDataDirty = true;
		return;
	});
}

void FLidarPointCloudOctree::ExecuteActionOnPointsByRay(TFunction<void(FLidarPointCloudPoint*)> Action, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly)
{
	PROCESS_BY_RAY_EX({ Action(Point); }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ApplyColorToAllPoints(const FColor& NewColor, const bool& bVisibleOnly)
{
	PROCESS_ALL_EX({ Point->Color = NewColor; }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ApplyColorToPointsInSphere(const FColor& NewColor, const FSphere& Sphere, const bool& bVisibleOnly)
{
	PROCESS_IN_SPHERE_EX({ Point->Color = NewColor; }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ApplyColorToPointsInBox(const FColor& NewColor, const FBox& Box, const bool& bVisibleOnly)
{
	PROCESS_IN_BOX_EX({ Point->Color = NewColor; }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::ApplyColorToFirstPointByRay(const FColor& NewColor, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly)
{
	PROCESS_BY_RAY({
		Point->Color = NewColor;
		CurrentNode->bRenderDataDirty = true;
		return;
	});
}

void FLidarPointCloudOctree::ApplyColorToPointsByRay(const FColor& NewColor, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly)
{
	PROCESS_BY_RAY_EX({ Point->Color = NewColor; }, { CurrentNode->bRenderDataDirty = true; });
}

void FLidarPointCloudOctree::MarkPointVisibilityDirty()
{
	ITERATE_NODES(
	{
		CurrentNode->bVisibilityDirty = true;
		CurrentNode->bRenderDataDirty = true;
	}, true);
}

void FLidarPointCloudOctree::MarkRenderDataDirty()
{
	ITERATE_NODES({ CurrentNode->bRenderDataDirty = true; }, true);
}

void FLidarPointCloudOctree::MarkRenderDataInSphereDirty(const FSphere& Sphere)
{
	const FBox Box(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));
	ITERATE_NODES({ CurrentNode->bRenderDataDirty = true; }, NODE_IN_BOX);
}

void FLidarPointCloudOctree::MarkRenderDataInConvexVolumeDirty(const FConvexVolume& ConvexVolume)
{
	ITERATE_NODES({ CurrentNode->bRenderDataDirty = true; }, NODE_IN_CONVEX_VOLUME);
}

#if WITH_EDITOR
void FLidarPointCloudOctree::SelectByConvexVolume(const FConvexVolume& ConvexVolume, bool bAdditive, bool bVisibleOnly)
{
	ITERATE_NODES(
	{
		// This will be used to determine if the payload should be released once convex check is complete
		const bool bHadData = CurrentNode->bHasData;
		bool bModified = false;

		// Proactively flag as selected to avoid async release of the node's data
		const bool bHadSelection = CurrentNode->bHasSelection;
		CurrentNode->bHasSelection = true;

		PROCESS_IN_CONVEX_VOLUME_BODY(
		{
			bModified = true;
			Point->bSelected = bAdditive;
		}, _RO)

		if(bModified)
		{
			CurrentNode->bRenderDataDirty = true;
		}
		else
		{
			CurrentNode->bHasSelection = bHadSelection;

			// Only attempt to release payload if the node didn't have any data loaded before
			if(!bHadData)
			{
				CurrentNode->ReleaseData();
			}
		}
	}, NODE_IN_CONVEX_VOLUME);
}

void FLidarPointCloudOctree::SelectBySphere(const FSphere& Sphere, bool bAdditive, bool bVisibleOnly)
{
	const FBox Box(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));
	const float RadiusSq = Sphere.W * Sphere.W;
	
	ITERATE_NODES(
	{
		// This will be used to determine if the payload should be released once convex check is complete
		const bool bHadData = CurrentNode->bHasData;
		bool bModified = false;

		// Proactively flag as selected to avoid async release of the node's data
		const bool bHadSelection = CurrentNode->bHasSelection;
		CurrentNode->bHasSelection = true;

		PROCESS_IN_SPHERE_BODY(
		{
			bModified = true;
			Point->bSelected = bAdditive;
		}, _RO)

		if(bModified)
		{
			CurrentNode->bRenderDataDirty = true;
		}
		else
		{
			CurrentNode->bHasSelection = bHadSelection;

			// Only attempt to release payload if the node didn't have any data loaded before
			if(!bHadData)
			{
				CurrentNode->ReleaseData();
			}
		}
	}, NODE_IN_BOX);
}

void FLidarPointCloudOctree::HideSelected()
{
	ITERATE_SELECTED({
		Point->bVisible = false;
		Point->bSelected = false;
	}, {
		CurrentNode->bHasSelection = false;
		CurrentNode->bCanReleaseData = false;
		CurrentNode->bRenderDataDirty = true;
		CurrentNode->bVisibilityDirty = true;
	});
}

void FLidarPointCloudOctree::DeleteSelected()
{
	ITERATE_SELECTED_NODES({
		const int32 NumElements = CurrentNode->GetNumPoints();
		FLidarPointCloudPoint* Start = CurrentNode->GetData();
		
		int64 NumRemoved = 0;

		for (FLidarPointCloudPoint* P = Start, *DataEnd = Start + NumElements; P != DataEnd; ++P)
		{
			if (P->bSelected)
			{
				CurrentNode->Data.RemoveAtSwap(P - Start, EAllowShrinking::No);
				++NumRemoved;
				--DataEnd;
				--P;
			}
		}

		CurrentNode->AddPointCount(-NumRemoved);

		// Reduce space usage
		CurrentNode->Data.Shrink();

		// Copy the updated array back to the BulkData
		CurrentNode->NumPoints = CurrentNode->Data.Num();

		CurrentNode->bHasSelection = false;
		CurrentNode->bCanReleaseData = false;
		CurrentNode->bRenderDataDirty = true;
		CurrentNode->bVisibilityDirty = true;
	});
}

void FLidarPointCloudOctree::InvertSelection()
{
	ITERATE_NODES(
	{
		// This will be used to determine if the payload should be released once invert is complete
		bool bNewHasSelection = false;
		
		FOR_RO(Point, CurrentNode)
		{
			Point->bSelected = !Point->bSelected;
			if(Point->bSelected)
			{
				bNewHasSelection = true;
			}
		}

		CurrentNode->bHasSelection = bNewHasSelection;
		CurrentNode->bRenderDataDirty = true;

		// Only attempt to release payload if the node isn't also used by rendering
		if(!bNewHasSelection && !CurrentNode->bInUse)
		{
			CurrentNode->ReleaseData();
		}
	}, true);
}

int64 FLidarPointCloudOctree::NumSelectedPoints() const
{
	int64 Count = 0;
	ITERATE_SELECTED({ ++Count; }, {});
	return Count;
}

bool FLidarPointCloudOctree::HasSelectedPoints() const
{
	ITERATE_SELECTED({ return true; }, {});
	return false;
}

void FLidarPointCloudOctree::GetSelectedPoints(TArray64<FLidarPointCloudPoint*>& SelectedPoints) const
{
	ITERATE_SELECTED({ SelectedPoints.Add(Point); }, {});
}

template <typename T>
void FLidarPointCloudOctree::GetSelectedPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FTransform& Transform) const
{
	const FTransform3f Transform3F = (FTransform3f)Transform;

	ITERATE_SELECTED(
	{
		SelectedPoints.Add(Point->Transform(Transform3F));
	},{});
}

void FLidarPointCloudOctree::GetSelectedPointsAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const
{
	GetSelectedPointsAsCopies_Internal(SelectedPoints, Transform);
}

void FLidarPointCloudOctree::GetSelectedPointsAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const
{
	GetSelectedPointsAsCopies_Internal(SelectedPoints, Transform);
}

void FLidarPointCloudOctree::GetSelectedPointsInBox(TArray64<const FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box) const
{
	constexpr bool bVisibleOnly = false;
	
	SelectedPoints.Reset(); 
	PROCESS_IN_BOX({
		if(Point->bSelected)
		{
			SelectedPoints.Add(Point);
		}
	});
}

void FLidarPointCloudOctree::ClearSelection()
{
	ITERATE_SELECTED({
		Point->bSelected = false; 
	}, {
		CurrentNode->bRenderDataDirty = true;
		CurrentNode->bHasSelection = false;

		// Only attempt to release payload if the node isn't also used by rendering
		if(!CurrentNode->bInUse)
		{
			CurrentNode->ReleaseData();
		}
	});
}

void FLidarPointCloudOctree::BuildStaticMeshBuffersForSelection(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform)
{
	if(CellSize == 0)
	{
		CellSize = GetEstimatedPointSpacing() * 1.1f;
	}
	
	LidarPointCloudMeshing::BuildStaticMeshBuffers(this, CellSize, true, OutMeshBuffers, Transform);
}
#endif // WITH_EDITOR

void FLidarPointCloudOctree::MarkRenderDataInFrustumDirty(const FConvexVolume& Frustum)
{
	MarkRenderDataInConvexVolumeDirty(Frustum);
}

void FLidarPointCloudOctree::Initialize(const FVector3f& InExtent)
{
	const bool bValidExtent = InExtent.X > 0 && InExtent.Y > 0 && InExtent.Z > 0;
	if (!bValidExtent)
	{
		PC_ERROR("Provided bounds are incorrect: %s", *InExtent.ToString());
		return;
	}

	Extent = InExtent;
	const FVector3f UniformExtent = FVector3f(InExtent.GetMax());
	
	MaxBucketSize = GetDefault<ULidarPointCloudSettings>()->MaxBucketSize;
	NodeGridResolution = GetDefault<ULidarPointCloudSettings>()->NodeGridResolution;

	// Pre-calculate the shared per LOD data
	for (int32 i = 0; i < SharedData.Num(); i++)
	{
		SharedData[i] = FSharedLODData(UniformExtent / FMath::Pow(2.f, i));
		NodeCount[i].Reset();
		PointCount[i].Reset();
	}

	Empty(true);

	bIsFullyLoaded = false;
}

void FLidarPointCloudOctree::InsertPoint(const FLidarPointCloudPoint* Point, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation)
{
	InsertPoints_Internal(Point, 1, DuplicateHandling, bRefreshPointsBounds, Translation);
}

void FLidarPointCloudOctree::InsertPoints(FLidarPointCloudPoint** Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation)
{
	InsertPoints_Internal(Points, Count, DuplicateHandling, bRefreshPointsBounds, Translation);
}

void FLidarPointCloudOctree::InsertPoints(const FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation)
{
	InsertPoints_Internal(Points, Count, DuplicateHandling, bRefreshPointsBounds, Translation);
}

void FLidarPointCloudOctree::InsertPoints(FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation)
{
	InsertPoints_Internal(Points, Count, DuplicateHandling, bRefreshPointsBounds, Translation);
}

void FLidarPointCloudOctree::RemovePoint(const FLidarPointCloudPoint* Point)
{
	if (!Point)
	{
		return;
	}
	
	int32 Index = -1;
	FLidarPointCloudOctreeNode* CurrentNode = Root;
	while (CurrentNode)
	{
		const FLidarPointCloudPoint* RESTRICT Start = CurrentNode->GetData();
		const int32 NumElements = CurrentNode->GetNumPoints();
		for (const FLidarPointCloudPoint* RESTRICT P = Start, *RESTRICT DataEnd = Start + NumElements; P != DataEnd; ++P)
		{
			if (P == Point)
			{
				Index = P - Start;
				break;
			}
		}

		if (Index > INDEX_NONE)
		{
			RemovePoint_Internal(CurrentNode, Index);
			break;
		}
		else
		{
			const FVector3f CenterRelativeLocation = Point->Location - CurrentNode->Center;
			CurrentNode = CurrentNode->GetChildNodeAtLocation((CenterRelativeLocation.X > 0 ? 4 : 0) + (CenterRelativeLocation.Y > 0 ? 2 : 0) + (CenterRelativeLocation.Z > 0));
		}
	}

	RefreshBounds();
}

void FLidarPointCloudOctree::RemovePoint(FLidarPointCloudPoint Point)
{
	int32 Index = -1;
	FLidarPointCloudOctreeNode* CurrentNode = Root;
	while (CurrentNode)
	{
		const FLidarPointCloudPoint* RESTRICT Start = CurrentNode->GetData();
		const int32 NumElements = CurrentNode->GetNumPoints();
		for (const FLidarPointCloudPoint* RESTRICT P = Start, *RESTRICT DataEnd = Start + NumElements; P != DataEnd; ++P)
		{
			if (*P == Point)
			{
				Index = P - Start;
				break;
			}
		}

		if (Index != INDEX_NONE)
		{
			RemovePoint_Internal(CurrentNode, Index);
			break;
		}
		else
		{
			const FVector3f CenterRelativeLocation = Point.Location - CurrentNode->Center;
			CurrentNode = CurrentNode->GetChildNodeAtLocation((CenterRelativeLocation.X > 0 ? 4 : 0) + (CenterRelativeLocation.Y > 0 ? 2 : 0) + (CenterRelativeLocation.Z > 0));
		}
	}

	RefreshBounds();
}

void FLidarPointCloudOctree::RemovePoints(TArray<FLidarPointCloudPoint*>& Points)
{
	RemovePoints_Internal(Points);
}

void FLidarPointCloudOctree::RemovePoints(TArray64<FLidarPointCloudPoint*>& Points)
{
	RemovePoints_Internal(Points);
}

template <typename T>
void FLidarPointCloudOctree::RemovePoints_Internal(TArray<FLidarPointCloudPoint*, T>& Points)
{
	if (Points.Num() == 0)
	{
		return;
	}

	for (FLidarPointCloudPoint** Point = Points.GetData(), ** DataEnd = Point + Points.Num(); Point != DataEnd; ++Point)
	{
		(*Point)->bMarkedForDeletion = true;
	}

	ITERATE_NODES(
	{
		FLidarPointCloudPoint* Start = CurrentNode->GetData();
		const int32 NumElements = CurrentNode->GetNumPoints();
	
		bool bHasPointsToRemove = false;
		for (FLidarPointCloudPoint* P = Start, *DataEnd = Start + NumElements; P != DataEnd; ++P)
		{
			if (P->bMarkedForDeletion)
			{
				bHasPointsToRemove = true;
				break;
			}
		}

		if (bHasPointsToRemove)
		{
			int64 NumRemoved = 0;

			for (FLidarPointCloudPoint* P = Start, *DataEnd = Start + NumElements; P != DataEnd; ++P)
			{
				if (P->bMarkedForDeletion)
				{
					CurrentNode->Data.RemoveAtSwap(P - Start, EAllowShrinking::No);
					++NumRemoved;
					--DataEnd;
					--P;
				}
			}

			// #todo: Fetch points from child nodes / padding points to fill the gap

			CurrentNode->AddPointCount(-NumRemoved);

			// Reduce space usage
			CurrentNode->Data.Shrink();

			// Copy the updated array back to the BulkData
			CurrentNode->NumPoints = CurrentNode->Data.Num();
			CurrentNode->bCanReleaseData = false;

			// Sort points to speed up rendering
			CurrentNode->SortVisiblePoints();

			CurrentNode->bRenderDataDirty = true;
		}
	}, true);

	RefreshBounds();
}

void FLidarPointCloudOctree::RemovePointsInSphere(const FSphere& Sphere, const bool& bVisibleOnly)
{
	// #todo: This can be optimized by removing points inline
	TArray<FLidarPointCloudPoint*> SelectedPoints;
	GetPointsInSphere(SelectedPoints, Sphere, bVisibleOnly);
	RemovePoints(SelectedPoints);
}

void FLidarPointCloudOctree::RemovePointsInBox(const FBox& Box, const bool& bVisibleOnly)
{
	// #todo: This can be optimized by removing points inline
	TArray<FLidarPointCloudPoint*> SelectedPoints;
	GetPointsInBox(SelectedPoints, Box, bVisibleOnly);
	RemovePoints(SelectedPoints);
}

void FLidarPointCloudOctree::RemoveFirstPointByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly)
{
	RemovePoint(RaycastSingle(Ray, Radius, bVisibleOnly));
}

void FLidarPointCloudOctree::RemovePointsByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly)
{
	// #todo: This can be optimized by removing points inline
	TArray<FLidarPointCloudPoint*> SelectedPoints;
	RaycastMulti(Ray, Radius, bVisibleOnly, SelectedPoints);
	RemovePoints(SelectedPoints);
}

void FLidarPointCloudOctree::RemoveHiddenPoints()
{
	ITERATE_NODES(
	{
		FLidarPointCloudPoint* Start = CurrentNode->GetData();
		const int32 NumElements = CurrentNode->GetNumPoints();

		bool bHasPointsToRemove = false;
		for (FLidarPointCloudPoint* P = Start, *DataEnd = Start + NumElements; P != DataEnd; ++P)
		{
			if (!P->bVisible)
			{
				bHasPointsToRemove = true;
				break;
			}
		}

		if (bHasPointsToRemove)
		{
			int64 NumRemoved = 0;

			for (FLidarPointCloudPoint* P = Start, *DataEnd = Start + NumElements; P != DataEnd; ++P)
			{
				if (!P->bVisible)
				{
					CurrentNode->Data.RemoveAtSwap(P - Start, EAllowShrinking::No);
					++NumRemoved;
					--DataEnd;
					--P;
				}
			}

			// #todo: Fetch points from child nodes / padding points to fill the gap

			CurrentNode->AddPointCount(-NumRemoved);

			// Reduce space usage
			CurrentNode->Data.Shrink();

			// Copy the updated array back to the BulkData
			CurrentNode->NumPoints = CurrentNode->Data.Num();
			CurrentNode->bCanReleaseData = false;

			// Set visibility data
			CurrentNode->NumVisiblePoints = CurrentNode->GetNumPoints();
			CurrentNode->bVisibilityDirty = false;

			CurrentNode->bRenderDataDirty = true;
		}
	}, true);

	RefreshBounds();
}

void FLidarPointCloudOctree::ResetNormals()
{
	ITERATE_NODES({
		FOR(Point, CurrentNode)
		{
			Point->Normal.Reset();
		}
		
		CurrentNode->bRenderDataDirty = true;
	}, true);
}

void FLidarPointCloudOctree::CalculateNormals(FThreadSafeBool* bCancelled, int32 Quality, float Tolerance, TArray64<FLidarPointCloudPoint*>* InPointSelection)
{
	FScopeBenchmarkTimer Timer("Calculate Normals");

	TArray64<FLidarPointCloudPoint*> EmptyArray;

	if (InPointSelection)
	{
		for (FLidarPointCloudPoint** Point = InPointSelection->GetData(), **DataEnd = Point + InPointSelection->Num(); Point != DataEnd; ++Point)
		{
			(*Point)->Normal.Reset();
		}

		MarkRenderDataDirty();
	}
	else
	{
		ResetNormals();
	}
	
	LidarPointCloudMeshing::CalculateNormals(this, bCancelled, Quality, Tolerance, InPointSelection ? *InPointSelection : EmptyArray);
}

void FLidarPointCloudOctree::Empty(bool bDestroyNodes)
{
	FScopeLock OctreeLock(&DataLock);
	MarkTraversalOctreesForInvalidation();

	if (bDestroyNodes)
	{
		// Reset node counters
		for (FThreadSafeCounter& Count : NodeCount)
		{
			Count.Reset();
		}
		
		// Perform the actual deletion on the RT, so there is no concurrent access issues
		ENQUEUE_RENDER_COMMAND(LidarPointCloudOctreeNode_DeleteOldData)([OldRoot = Root](FRHICommandListImmediate& RHICmdList)
		{
			delete OldRoot;
		});
		
		Root = new FLidarPointCloudOctreeNode(this, 0);

		QueuedNodes.Empty();
		NodesInUse.Reset();
	}
	else
	{
		Root->Empty(true);
	}
	
	// Reset point counters
	for (FThreadSafeCounter64& Count : PointCount)
	{
		Count.Reset();
	}
}

void FLidarPointCloudOctree::UnregisterTraversalOctree(FLidarPointCloudTraversalOctree* TraversalOctree)
{
	if (TraversalOctree)
	{
		bool bRemoved = false;

		for (int32 i = 0; i < LinkedTraversalOctrees.Num(); ++i)
		{
			if (TSharedPtr<FLidarPointCloudTraversalOctree, ESPMode::ThreadSafe> TO = LinkedTraversalOctrees[i].Pin())
			{
				if (TO.Get() == TraversalOctree)
				{
					LinkedTraversalOctrees.RemoveAtSwap(i--);
					bRemoved = true;
				}
			}
			// Remove null
			else
			{
				LinkedTraversalOctrees.RemoveAtSwap(i--);
			}
		}

		// If nothing is using this Octree, release all non-persistent nodes
		if (bRemoved && LinkedTraversalOctrees.Num() == 0)
		{
			FScopeLock Lock(&DataLock);
			ReleaseAllNodes(false);
		}
	}
}

void FLidarPointCloudOctree::StreamNodes(TArray<FLidarPointCloudOctreeNode*>& Nodes, const float& CurrentTime)
{
	const float BulkDataLifetime = CurrentTime + GetDefault<ULidarPointCloudSettings>()->CachedNodeLifetime;

	for (FLidarPointCloudOctreeNode* Node : Nodes)
	{
		// Refresh lifetime of the BulkData
		Node->BulkDataLifetime = BulkDataLifetime;

		// Enqueue for processing
		QueuedNodes.Enqueue(Node);
	}

	// Only one process at a time
	if (!bStreamingBusy)
	{
		bStreamingBusy = true;

		// Add previously queued nodes
		FLidarPointCloudOctreeNode* QueuedNode;
		while (QueuedNodes.Dequeue(QueuedNode))
		{
			if (!QueuedNode->bInUse)
			{
				NodesInUse.Add(QueuedNode);
			}
		}

		// Process nodes' streaming requests
		// If the release lock has been acquired, release unused nodes in one go
		// Otherwise, just load the ones requested
		FScopeTryLock ReleaseLock(&DataReleaseLock);
		if(ReleaseLock.IsLocked())
		{
			for (int32 i = 0; i < NodesInUse.Num(); ++i)
			{
				FLidarPointCloudOctreeNode* Node = NodesInUse[i];

				// Check if the node should still be in use
				Node->bInUse = Node->BulkDataLifetime >= CurrentTime;

				// If node is alive, make sure it has data loaded
				if (Node->bInUse)
				{
					Node->GetData();
				}
				// ... otherwise, release it
				else
				{
					Node->ReleaseData();				
					NodesInUse.RemoveAtSwap(i--, EAllowShrinking::No);
				}
			}
		}
		else
		{
			for (FLidarPointCloudOctreeNode* Node : NodesInUse)
			{
				if (Node->BulkDataLifetime >= CurrentTime)
				{
					Node->GetData();
				}
			}
		}

		bStreamingBusy = false;
	}
}

void FLidarPointCloudOctree::LoadAllNodes(bool bLoadPersistently)
{
	if (bLoadPersistently)
	{
		ITERATE_NODES({ CurrentNode->GetPersistentData(); }, true);
		bIsFullyLoaded = true;
	}
	else
	{
		ITERATE_NODES({ CurrentNode->GetData(); }, true);
	}
}

void FLidarPointCloudOctree::ReleaseAllNodes(bool bIncludePersistent)
{
	ITERATE_NODES({ CurrentNode->ReleaseData(bIncludePersistent); }, true);

	if (bIncludePersistent)
	{
		bIsFullyLoaded = false;
	}
}

bool FLidarPointCloudOctree::IsOptimizedForDynamicData() const
{
	return Owner && Owner->IsOptimizedForDynamicData();
}

void FLidarPointCloudOctree::OptimizeForDynamicData()
{
	if (!IsOptimizedForDynamicData())
	{
		// Move all data from the nodes
		TArray<FLidarPointCloudPoint> SourceData;
		SourceData.Reserve(GetNumPoints());
		
		ITERATE_NODES(
		{
			SourceData.Append(CurrentNode->Data);
			CurrentNode->Data.Empty();
		}, true);

		// Destroy current tree structure
		Empty(true);

		bIsFullyLoaded = true;

		// Insert data back into the tree
		InsertPoints_Internal(SourceData.GetData(), SourceData.Num(), ELidarPointCloudDuplicateHandling::Ignore, false, FVector3f::ZeroVector);
	}
}

void FLidarPointCloudOctree::OptimizeForStaticData()
{
	if (IsOptimizedForDynamicData())
	{
		// Move data out of the root node
		TArray<FLidarPointCloudPoint> SourceData = MoveTemp(Root->Data);
		
		// Destroy current tree structure
		Empty(true);

		bIsFullyLoaded = true;

		// Insert data back into the tree
		InsertPoints_Internal(SourceData.GetData(), SourceData.Num(), ELidarPointCloudDuplicateHandling::Ignore, false, FVector3f::ZeroVector);
	}
}

void FLidarPointCloudOctree::RefreshAllocatedSize()
{
	FScopeTryLock Lock(&DataLock);
	if (!Lock.IsLocked())
	{
		return;
	}

	PreviousPointCount = GetNumPoints();
	PreviousNodeCount = GetNumNodes();

	PreviousAllocatedStructureSize = sizeof(FLidarPointCloudOctree);

	PreviousAllocatedStructureSize += SharedData.GetAllocatedSize();
	PreviousAllocatedStructureSize += PointCount.GetAllocatedSize();

	PreviousAllocatedSize = PreviousAllocatedStructureSize + Root->GetAllocatedSize(true, true);
	PreviousAllocatedStructureSize += Root->GetAllocatedSize(true, false);
}

template <typename T>
void FLidarPointCloudOctree::InsertPoints_Internal(T Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation)
{
	Root->InsertPoints(Points, Count, DuplicateHandling, Translation);
	MarkTraversalOctreesForInvalidation();
	if (bRefreshPointsBounds)
	{
		RefreshBounds();
	}
}

void FLidarPointCloudOctree::RemovePoint_Internal(FLidarPointCloudOctreeNode* Node, int32 Index)
{
	Node->GetPersistentData();
	Node->AddPointCount(-1);
	Node->bRenderDataDirty = true;
	Node->Data.RemoveAt(Index);
	Node->NumPoints = Node->Data.Num();

	// #todo: Fetch points from child nodes / padding points to fill the gap
}

void FLidarPointCloudOctree::MarkTraversalOctreesForInvalidation()
{
	for (int32 i = 0; i < LinkedTraversalOctrees.Num(); ++i)
	{
		if (TSharedPtr<FLidarPointCloudTraversalOctree, ESPMode::ThreadSafe> TraversalOctree = LinkedTraversalOctrees[i].Pin())
		{
			TraversalOctree->bValid = false;
		}
		// Remove null
		else
		{
			LinkedTraversalOctrees.RemoveAtSwap(i--);
		}
	}
}

void FLidarPointCloudOctree::Serialize(FArchive& Ar)
{
	// Extent
	{
		FVector3f NodesExtent = SharedData[0].Extent;

		if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) > 16)
		{
			Ar << NodesExtent;
		}
		else
		{
			FBox Bounds;
			Ar << Bounds;
			NodesExtent = (FVector3f)Bounds.GetExtent();
		}

		if (Ar.IsLoading())
		{
			Initialize(NodesExtent);
		}

		if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) >= 20)
		{
			Ar << Extent;
		}
	}

	// Collision Mesh data
	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) > 13)
	{
		FTriMeshCollisionData Dummy;
		FTriMeshCollisionData* CollisionMeshPtr = Ar.IsCooking() ? &Dummy : &CollisionMesh;

		Ar << CollisionMeshPtr->Vertices;

		int32 NumIndices = CollisionMeshPtr->Indices.Num();
		Ar << NumIndices;

		if (Ar.IsLoading())
		{
			CollisionMeshPtr->Indices.AddUninitialized(NumIndices);
		}

		Ar.Serialize(CollisionMeshPtr->Indices.GetData(), NumIndices * sizeof(FTriIndices));
	}

	// Used for backwards compatibility with pre-streaming formats
	if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) < 16)
	{
		TArray<FLidarPointCloudPoint_Legacy> DataArray;

		TArray<FLidarPointCloudOctreeNode*> Nodes;
		Nodes.Add(Root);
		while (Nodes.Num())
		{
			FLidarPointCloudOctreeNode* CurrentNode = Nodes.Pop(EAllowShrinking::No);

			Ar << CurrentNode->LocationInParent << CurrentNode->Center;
			
			CurrentNode->Data.Empty();
			Ar << DataArray; // AllocatedPoints
			CurrentNode->Data.Append(DataArray);
			Ar << DataArray; // PaddingPoints
			CurrentNode->Data.Append(DataArray);

			CurrentNode->bHasData = true;
			CurrentNode->bCanReleaseData = false;

			int32 NumChildren = CurrentNode->Children.Num();
			Ar << NumChildren;

			PointCount[CurrentNode->Depth].Add(CurrentNode->NumPoints);
			CurrentNode->NumVisiblePoints = CurrentNode->NumPoints;

			CurrentNode->Children.AddUninitialized(NumChildren);
			for (int32 i = 0; i < NumChildren; ++i)
			{
				CurrentNode->Children[i] = new FLidarPointCloudOctreeNode(this, CurrentNode->Depth + 1);
			}

			for (int32 i = NumChildren - 1; i >= 0; --i)
			{
				Nodes.Add(CurrentNode->Children[i]);
			}
		}
	}
	// Used for backwards compatibility with BulkData
	else if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) < 20)
	{
		ITERATE_NODES({
			// Pre-normals
			if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) < 19)
			{
				FLidarPointCloudBulkData_Legacy LegacyBulkData(sizeof(FLidarPointCloudPoint_Legacy));

				// Get the legacy data
				void* TempData = nullptr;
				LegacyBulkData.Serialize(Ar, Owner);
				LegacyBulkData.GetCopy(&TempData);

				// Allocate the data buffer
				CurrentNode->NumPoints = LegacyBulkData.GetElementCount();
				CurrentNode->Data.SetNumUninitialized(CurrentNode->NumPoints);
				FLidarPointCloudPoint* DataPtr = CurrentNode->Data.GetData();

				// Copy the legacy data
				for (FLidarPointCloudPoint_Legacy* Data = (FLidarPointCloudPoint_Legacy*)TempData, *DataEnd = Data + CurrentNode->NumPoints; Data != DataEnd; ++Data, ++DataPtr)
				{
					*DataPtr = *Data;
				}

				// Release the legacy data
				FMemory::Free(TempData);
			}
			else
			{
				FLidarPointCloudBulkData_Legacy LegacyBulkData(sizeof(FLidarPointCloudPoint));
				LegacyBulkData.Serialize(Ar, Owner);

				CurrentNode->NumPoints = LegacyBulkData.GetElementCount();
				CurrentNode->Data.SetNumUninitialized(CurrentNode->NumPoints);
				void* DataPtr = CurrentNode->Data.GetData();
				LegacyBulkData.GetCopy(&DataPtr);
			}

			CurrentNode->bHasData = true;
			CurrentNode->bCanReleaseData = false;

			int32 NumChildren = CurrentNode->Children.Num();
			Ar << CurrentNode->LocationInParent << CurrentNode->Center << NumChildren;

			PointCount[CurrentNode->Depth].Add(CurrentNode->NumPoints);
			CurrentNode->NumVisiblePoints = CurrentNode->NumPoints;

			CurrentNode->Children.AddUninitialized(NumChildren);
			for (int32 i = 0; i < NumChildren; i++)
			{
				CurrentNode->Children[i] = new FLidarPointCloudOctreeNode(this, CurrentNode->Depth + 1);
			}
		}, true);
	}
	else
	{
		// Make sure to load all data persistently before saving, as the linker will detach
		if (Ar.IsSaving())
		{
			LoadAllNodes(true);
			CloseReadHandle();
		}

		int64 BulkDataOffset = 0;

		ITERATE_NODES({
			int32 NumChildren = CurrentNode->Children.Num();
			Ar << CurrentNode->LocationInParent << CurrentNode->Center << NumChildren << CurrentNode->NumPoints;

			CurrentNode->BulkDataSize = CurrentNode->NumPoints * sizeof(FLidarPointCloudPoint);
			CurrentNode->BulkDataOffset = BulkDataOffset;
			BulkDataOffset += CurrentNode->BulkDataSize;
			Ar << CurrentNode->BulkDataSize << CurrentNode->BulkDataOffset;

			if (Ar.IsSaving())
			{
				// Make sure the points are in optimized order before saving
				CurrentNode->SortVisiblePoints();
			}
			else
			{
				PointCount[CurrentNode->Depth].Add(CurrentNode->NumPoints);
				CurrentNode->NumVisiblePoints = CurrentNode->NumPoints;

				// Build sub-nodes
				CurrentNode->Children.AddUninitialized(NumChildren);
				for (int32 i = 0; i < NumChildren; i++)
				{
					CurrentNode->Children[i] = new FLidarPointCloudOctreeNode(this, CurrentNode->Depth + 1);
				}
			}
		}, true);
		
#if WITH_EDITOR
		if(Ar.HasAnyPortFlags(PPF_Duplicate))
		{
			if(Ar.IsLoading())
			{
				ITERATE_NODES({
					CurrentNode->Data.SetNumUninitialized(CurrentNode->NumPoints);
					Ar.Serialize(CurrentNode->Data.GetData(), CurrentNode->BulkDataSize);
					CurrentNode->bCanReleaseData = false;
					CurrentNode->bRenderDataDirty = true;
					CurrentNode->bHasData = true;
				}, true);	
			}
			else
			{
				ITERATE_NODES({
					Ar.Serialize(CurrentNode->Data.GetData(), CurrentNode->BulkDataSize);
				}, true);	
			}
		}
		else if(Ar.IsSaving())
		{
			SavingBulkData.Serialize(Ar, Owner);
		}
		else
#endif
		{
			BulkData.Serialize(Ar, Owner);
		}
	}

	// Legacy Points Extent
	{
		if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) < 17)
		{
			FBox PointsBounds;

			if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) > 14)
			{
				Ar << PointsBounds;
			}

			if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) > 15)
			{
				Extent = (FVector3f)PointsBounds.GetExtent();
				Owner->LocationOffset = PointsBounds.GetCenter();
			}
			else
			{
				RefreshBounds();
			}
		}
		else if (Ar.CustomVer(ULidarPointCloud::PointCloudFileGUID) < 20)
		{
			Ar << Extent;
		}
	}
}

void FLidarPointCloudOctree::SerializeBulkData(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		const bool bReleaseData = !!Owner->GetPackage()->GetLinker();
		
		ITERATE_NODES({
			Ar.Serialize(CurrentNode->Data.GetData(), CurrentNode->BulkDataSize);
			if(bReleaseData)
			{
				CurrentNode->ReleaseData(true);
			}
		}, true);
	}
}

IAsyncReadFileHandle* FLidarPointCloudOctree::GetReadHandle()
{
	if(!ReadHandle)
	{
		ReadHandle = BulkData.OpenAsyncReadHandle();
	}

	return ReadHandle;
}

void FLidarPointCloudOctree::CloseReadHandle()
{
	if (ReadHandle)
	{
		delete ReadHandle;
		ReadHandle = nullptr;
	}
}

void FLidarPointCloudOctree::StreamNodeData(FLidarPointCloudOctreeNode* Node)
{
	if (GetReadHandle())
	{
		Node->Data.SetNumUninitialized(Node->NumPoints);		
		IAsyncReadRequest* ReadRequest = ReadHandle->ReadRequest(
			BulkData.GetBulkDataOffsetInFile() + Node->BulkDataOffset,
			Node->BulkDataSize,
			AIOP_Normal,
			nullptr,
			(uint8*)Node->Data.GetData());
		ReadRequest->WaitCompletion();
		delete ReadRequest;
		Node->bHasData = true;
	}
	else
	{
		Node->bHasData = false;
		Node->Data.Empty();
	}
}

//////////////////////////////////////////////////////////// FLidarPointCloudBulkData

#if WITH_EDITOR
FLidarPointCloudOctree::FLidarPointCloudBulkData::FLidarPointCloudBulkData(FLidarPointCloudOctree* Octree)
{
	SerializeElementsCallback = [Octree](FArchive& Ar, void*, int64, EBulkDataFlags)
	{
		Octree->SerializeBulkData(Ar);
	};
#if !USE_RUNTIME_BULKDATA
	SerializeBulkDataElements = &SerializeElementsCallback;
#endif // !USE_RUNTIME_BULKDATA
	
	SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload | BULKDATA_Size64Bit);
}

void FLidarPointCloudOctree::FLidarPointCloudBulkData::Serialize(FArchive& Ar, UObject* InOwner)
{
	Lock(LOCK_READ_WRITE);
	Realloc(1,1);
	Unlock();
	SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload | BULKDATA_Size64Bit);

	FBulkData::Serialize(Ar, InOwner, false, 1, EFileRegionType::None);
}
#endif
//////////////////////////////////////////////////////////// FLidarPointCloudTraversalOctreeNode

FLidarPointCloudTraversalOctreeNode::FLidarPointCloudTraversalOctreeNode()
	: DataNode(nullptr)
	, Parent(nullptr)
	, bFullyContained(false)
{

}

void FLidarPointCloudTraversalOctreeNode::Build(FLidarPointCloudTraversalOctree* TraversalOctree, FLidarPointCloudOctreeNode* Node, const FTransform& LocalToWorld, const FVector3f& LocationOffset)
{
	Octree = TraversalOctree;
	DataNode = Node;
	Center = (FVector3f)LocalToWorld.TransformPosition((FVector)(Node->Center + LocationOffset));
	Depth = Node->Depth;

	Children.AddZeroed(Node->Children.Num());
	for (int32 i = 0; i < Children.Num(); i++)
	{
		if (Node->Children[i])
		{
			Children[i].Build(Octree, Node->Children[i], LocalToWorld, LocationOffset);
			Children[i].Parent = this;
		}
	}
}

void FLidarPointCloudTraversalOctreeNode::CalculateVirtualDepth(const TArray<float>& LevelWeights, const float& PointSizeBias)
{
	if (!IsAvailable())
	{
		VirtualDepth = 255;
		return;
	}

	const float& VDMultiplier = Octree->VirtualDepthMultiplier;

	TQueue<const FLidarPointCloudTraversalOctreeNode*> Nodes;
	const FLidarPointCloudTraversalOctreeNode* CurrentNode = nullptr;

	// Calculate virtual depth factor
	float VDFactor = 0;
	Nodes.Enqueue(this);
	while (Nodes.Dequeue(CurrentNode))
	{
		for (const FLidarPointCloudTraversalOctreeNode& Child : CurrentNode->Children)
		{
			if (Child.IsAvailable())
			{
				Nodes.Enqueue(&Child);
			}
		}

		float LocalVDFactor = CurrentNode->Depth * CurrentNode->DataNode->GetNumVisiblePoints() * LevelWeights[CurrentNode->Depth];

		if (CurrentNode != this && PointSizeBias > 0)
		{
			LocalVDFactor /= (CurrentNode->Parent->Children.Num() - 1) * PointSizeBias + 1;
		}

		VDFactor += LocalVDFactor;
	}

	// Calculate weighted number of visible points
	float NumPoints = 0;
	Nodes.Enqueue(this);
	while (Nodes.Dequeue(CurrentNode))
	{
		for (const FLidarPointCloudTraversalOctreeNode& Child : CurrentNode->Children)
		{
			if (Child.IsAvailable())
			{
				Nodes.Enqueue(&Child);
			}
		}

		NumPoints += CurrentNode->DataNode->GetNumVisiblePoints() * LevelWeights[CurrentNode->Depth];
	}

	// Calculate the Virtual Depth
	VirtualDepth = VDFactor / NumPoints * VDMultiplier;
}

//////////////////////////////////////////////////////////// FLidarPointCloudTraversalOctreeNodeSizeData

FLidarPointCloudTraversalOctreeNodeSizeData::FLidarPointCloudTraversalOctreeNodeSizeData(FLidarPointCloudTraversalOctreeNode* Node, const float& Size, const int32& ProxyIndex)
	: Node(Node)
	, Size(Size)
	, ProxyIndex(ProxyIndex)
{
}

//////////////////////////////////////////////////////////// FLidarPointCloudTraversalOctree

FLidarPointCloudTraversalOctree::FLidarPointCloudTraversalOctree(FLidarPointCloudOctree* Octree, const FTransform& LocalToWorld)
	: Octree(Octree)
	, bValid(false)
{	
	// Reset the tree
	Extents.Empty();
	RadiiSq.Empty();
	Root.~FLidarPointCloudTraversalOctreeNode();
	new (&Root) FLidarPointCloudTraversalOctreeNode();

	// Calculate properties
	NumLODs = Octree->GetNumLODs();

	VirtualDepthMultiplier = 255.0f / NumLODs;
	ReversedVirtualDepthMultiplier = NumLODs / 255.0f;

	const FVector3f Extent = Octree->SharedData[0].Extent;

	const FBox WorldBounds = FBox(-Extent, Extent).TransformBy(LocalToWorld);
	for (int32 i = 0; i < NumLODs; i++)
	{
		Extents.Emplace(i == 0 ? (FVector3f)WorldBounds.GetExtent() : Extents.Last() * 0.5f);
		RadiiSq.Emplace(FMath::Square(Extents.Last().Size()));
	}

	int64 NumPoints = 0;
	TArray<int64> PointCount;
	for (FThreadSafeCounter64& Count : Octree->PointCount)
	{
		PointCount.Add(Count.GetValue());
		NumPoints += Count.GetValue();
	}

	LevelWeights.AddZeroed(NumLODs);
	for (int32 i = 0; i < LevelWeights.Num(); i++)
	{
		LevelWeights[i] = (float)PointCount[i] / NumPoints;
	}

	// Star cloning the node data
	Root.Build(this, Octree->Root, LocalToWorld, (FVector3f)Octree->Owner->LocationOffset);

	bValid = NumLODs > 0;
}

void FLidarPointCloudTraversalOctree::CalculateVisibilityStructure(TArray<uint32>& OutData)
{
	FLidarPointCloudTraversalOctreeNode* CurrentNode = nullptr;
	TQueue<FLidarPointCloudTraversalOctreeNode*> Nodes;
	Nodes.Enqueue(&Root);

	uint16 NumNodesInQueue = 1;

	while (Nodes.Dequeue(CurrentNode))
	{
		uint32& Data = OutData[OutData.AddZeroed()];
		Data |= (0x00FFFF00 & (NumNodesInQueue-- << 8));
		Data |= 0xFF000000 & (CurrentNode->VirtualDepth << 24);

		for (uint8 i = 0; i < 8; ++i)
		{
			for (FLidarPointCloudTraversalOctreeNode& Child : CurrentNode->Children)
			{
				if (Child.bSelected && Child.DataNode->LocationInParent == i)
				{
					Nodes.Enqueue(&Child);
					Data |= 1 << i;
					++NumNodesInQueue;
				}
			}
		}
	}
}

void FLidarPointCloudTraversalOctree::CalculateLevelWeightsForSelectedNodes(TArray<float>& OutLevelWeights)
{
	FLidarPointCloudTraversalOctreeNode* CurrentNode = nullptr;
	TQueue<FLidarPointCloudTraversalOctreeNode*> Nodes;
	Nodes.Enqueue(&Root);

	OutLevelWeights.Empty();
	OutLevelWeights.AddZeroed(NumLODs);
	TArray<int64> PointCount;
	PointCount.AddZeroed(NumLODs);
	int64 NumPoints = 0;

	while (Nodes.Dequeue(CurrentNode))
	{
		for (FLidarPointCloudTraversalOctreeNode& Child : CurrentNode->Children)
		{
			if (Child.bSelected)
			{
				Nodes.Enqueue(&Child);
			}
		}

		const int32 NumPointsInNode = CurrentNode->DataNode->GetNumVisiblePoints();
		PointCount[CurrentNode->Depth] += NumPointsInNode;
		NumPoints += NumPointsInNode;
	}

	for (int32 i = 0; i < OutLevelWeights.Num(); i++)
	{
		OutLevelWeights[i] = NumPoints > 0 ? (float)PointCount[i] / NumPoints : 0;
	}
}

FLidarPointCloudTraversalOctree::~FLidarPointCloudTraversalOctree()
{
	if (bValid)
	{
		Octree->UnregisterTraversalOctree(this);
	}
}
