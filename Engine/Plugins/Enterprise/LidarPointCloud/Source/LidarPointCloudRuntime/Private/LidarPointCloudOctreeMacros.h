// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define FOR(ElemName, Node) for(FLidarPointCloudPoint* ElemName = Node->GetPersistentData(), *DataEnd##ElemName = ElemName + Node->GetNumPoints(); ElemName != DataEnd##ElemName; ++ElemName)
#define FOR_RO(ElemName, Node) for(FLidarPointCloudPoint* ElemName = Node->GetData(), *DataEnd##ElemName = ElemName + Node->GetNumPoints(); ElemName != DataEnd##ElemName; ++ElemName)

#define IS_VIS_CHECK_REQUIRED (bVisibleOnly && CurrentNode->NumVisiblePoints < CurrentNode->GetNumPoints())

#define NODE_IN_BOX (Box.Intersect(Child->GetBounds()))
#define NODE_IN_CONVEX_VOLUME (ConvexVolume.IntersectBox((FVector)Child->Center, (FVector)SharedData[Child->Depth].Extent))

#define ITERATE_NODES_BODY(Action, NodeTest, Const) \
{\
	TQueue<Const FLidarPointCloudOctreeNode*> Nodes;\
	Const FLidarPointCloudOctreeNode* CurrentNode = nullptr;\
	Nodes.Enqueue(Root);\
	while (Nodes.Dequeue(CurrentNode))\
	{\
		{ Action } \
		for (FLidarPointCloudOctreeNode* Child : CurrentNode->Children)\
		{\
			if (NodeTest)\
			Nodes.Enqueue(Child);\
		}\
	}\
}
#define ITERATE_NODES(Action, NodeTest) ITERATE_NODES_BODY(Action, NodeTest,  )
#define ITERATE_NODES_CONST(Action, NodeTest) ITERATE_NODES_BODY(Action, NodeTest, const)

#define ITERATE_SELECTED_NODES(NodeAction) \
ITERATE_NODES(\
{\
	if(CurrentNode->bHasSelection) { NodeAction }\
}, true);

#define ITERATE_SELECTED(PointAction, NodeAction) \
ITERATE_SELECTED_NODES(\
{\
	FOR_RO(Point, CurrentNode) { if(Point->bSelected) { PointAction } }\
	{ NodeAction }\
});

#define POINT_IN_BOX Box.IsInsideOrOn((FVector)Point->Location)
#define POINT_IN_SPHERE (POINT_IN_BOX && FVector3f::DistSquared(Point->Location, (FVector3f)Sphere.Center) <= RadiusSq)
#define POINT_IN_CONVEX_VOLUME ConvexVolume.IntersectSphere((FVector)Point->Location, 0)
#define POINT_BY_RAY Ray.Intersects(Point, RadiusSq)

#define PROCESS_BODY(Action, PointTest, Mode) \
{\
	/* If node fully inside the box - do not check individual points */\
	if (bNodeFullyContained)\
	{\
		if (!IS_VIS_CHECK_REQUIRED) { FOR##Mode(Point, CurrentNode) { Action } }\
		else { FOR##Mode(Point, CurrentNode) { if (Point->bVisible) { Action } } }\
	}\
	else\
	{\
		if (!IS_VIS_CHECK_REQUIRED) { FOR##Mode(Point, CurrentNode) { if (PointTest) { Action } } }\
		else { FOR##Mode(Point, CurrentNode) { if (Point->bVisible && PointTest) { Action } } }\
	}\
}

#define PROCESS_ALL_BODY(Action, Mode) \
{\
	if (!bVisibleOnly || CurrentNode->NumVisiblePoints > 0)\
	{\
		const bool bNodeFullyContained = true;\
		PROCESS_BODY(Action, true, Mode) \
	}\
}

#define PROCESS_IN_SPHERE_BODY_EXTERN(Octree, Action, Mode) \
{\
	if (!bVisibleOnly || CurrentNode->NumVisiblePoints > 0)\
	{\
		const bool bNodeFullyContained = CurrentNode->GetSphereBounds().IsInside(Sphere);\
		PROCESS_BODY(Action, POINT_IN_SPHERE, Mode) \
	}\
}
#define PROCESS_IN_SPHERE_BODY(Action, Mode) PROCESS_IN_SPHERE_BODY_EXTERN(this, Action, Mode)

#define PROCESS_IN_BOX_BODY_EXTERN(Octree, Action, Mode) \
{\
	if (!bVisibleOnly || CurrentNode->NumVisiblePoints > 0)\
	{\
		const bool bNodeFullyContained = Box.IsInsideOrOn((FVector)(CurrentNode->Center - Octree->SharedData[CurrentNode->Depth].Extent)) && Box.IsInsideOrOn((FVector)(CurrentNode->Center + Octree->SharedData[CurrentNode->Depth].Extent));\
		PROCESS_BODY(Action, POINT_IN_BOX, Mode) \
	}\
}
#define PROCESS_IN_BOX_BODY(Action, Mode) PROCESS_IN_BOX_BODY_EXTERN(this, Action, Mode)

#define PROCESS_IN_CONVEX_VOLUME_BODY(Action, Mode) \
{\
	if (!bVisibleOnly || CurrentNode->NumVisiblePoints > 0)\
	{\
		bool bNodeFullyContained;\
		ConvexVolume.IntersectBox((FVector)CurrentNode->Center, (FVector)SharedData[CurrentNode->Depth].Extent, bNodeFullyContained);\
		PROCESS_BODY(Action, POINT_IN_CONVEX_VOLUME, Mode) \
	}\
}

#define PROCESS_BY_RAY_BODY(Action, Mode) \
{\
	if (!bVisibleOnly || CurrentNode->NumVisiblePoints > 0)\
	{\
		if (Ray.Intersects(CurrentNode->GetBounds()))\
		{\
			if (!IS_VIS_CHECK_REQUIRED) { FOR##Mode(Point, CurrentNode) { if (POINT_BY_RAY) { Action } } }\
			else { FOR##Mode(Point, CurrentNode) { if (Point->bVisible && POINT_BY_RAY) { Action } } }\
\
			for (FLidarPointCloudOctreeNode* Child : CurrentNode->Children)\
			{\
				 Nodes.Enqueue(Child);\
			}\
		}\
	}\
}

#define PROCESS_IN_SPHERE_COMMON(Action) \
{\
	/* Build a box to quickly filter out the points - (IsInsideOrOn vs comparing DistSquared) */\
	const FBox Box(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));\
	const float RadiusSq = Sphere.W * Sphere.W;\
	Action\
}
#define PROCESS_IN_SPHERE(Action) { PROCESS_IN_SPHERE_COMMON(ITERATE_NODES(PROCESS_IN_SPHERE_BODY(Action,), NODE_IN_BOX)) }
#define PROCESS_IN_SPHERE_EX(Action, NodeAction) { PROCESS_IN_SPHERE_COMMON(ITERATE_NODES({PROCESS_IN_SPHERE_BODY(Action,)} {NodeAction}, NODE_IN_BOX)) }
#define PROCESS_IN_SPHERE_CONST(Action) { PROCESS_IN_SPHERE_COMMON(ITERATE_NODES_CONST(PROCESS_IN_SPHERE_BODY(Action, _RO), NODE_IN_BOX)) }
#define PROCESS_IN_SPHERE_EXTERN(Octree, Action) { PROCESS_IN_SPHERE_COMMON(ITERATE_NODES(PROCESS_IN_SPHERE_BODY_EXTERN(Octree, Action,), NODE_IN_BOX)) }


#define PROCESS_ALL(Action) { ITERATE_NODES(PROCESS_ALL_BODY(Action,), true) }
#define PROCESS_ALL_EX(Action, NodeAction) { ITERATE_NODES({PROCESS_ALL_BODY(Action,)} {NodeAction}, true) }
#define PROCESS_ALL_CONST(Action) { ITERATE_NODES_CONST(PROCESS_ALL_BODY(Action, _RO), true) }

#define PROCESS_IN_BOX(Action) { ITERATE_NODES(PROCESS_IN_BOX_BODY(Action,), NODE_IN_BOX) }
#define PROCESS_IN_BOX_EX(Action, NodeAction) { ITERATE_NODES({PROCESS_IN_BOX_BODY(Action,)} {NodeAction}, NODE_IN_BOX) }
#define PROCESS_IN_BOX_CONST(Action) { ITERATE_NODES_CONST(PROCESS_IN_BOX_BODY(Action, _RO), NODE_IN_BOX) }
#define PROCESS_IN_BOX_EXTERN(Octree, Action) { ITERATE_NODES(PROCESS_IN_BOX_BODY_EXTERN(Octree, Action,), NODE_IN_BOX) }

#define PROCESS_IN_CONVEX_VOLUME(Action) { ITERATE_NODES(PROCESS_IN_CONVEX_VOLUME_BODY(Action,), NODE_IN_CONVEX_VOLUME) }
#define PROCESS_IN_CONVEX_VOLUME_CONST(Action) { ITERATE_NODES_CONST(PROCESS_IN_CONVEX_VOLUME_BODY(Action, _RO), NODE_IN_CONVEX_VOLUME) }

#define PROCESS_BY_RAY_COMMON(Action)\
{\
	const float RadiusSq = Radius * Radius;\
	Action \
}
#define PROCESS_BY_RAY(Action) { PROCESS_BY_RAY_COMMON(ITERATE_NODES(PROCESS_BY_RAY_BODY(Action,), false)) }
#define PROCESS_BY_RAY_EX(Action, NodeAction) { PROCESS_BY_RAY_COMMON(ITERATE_NODES({PROCESS_BY_RAY_BODY(Action,)} {NodeAction}, false)) }
#define PROCESS_BY_RAY_CONST(Action) { PROCESS_BY_RAY_COMMON(ITERATE_NODES_CONST(PROCESS_BY_RAY_BODY(Action, _RO), false)) }