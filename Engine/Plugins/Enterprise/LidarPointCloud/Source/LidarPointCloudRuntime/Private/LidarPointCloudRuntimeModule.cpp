// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILidarPointCloudRuntimeModule.h"
#include "LidarPointCloudShared.h"
#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogLidarPointCloud);

class FLidarPointCloudRuntimeModule : public ILidarPointCloudRuntimeModule
{
	// Begin IModuleInterface Interface
	virtual void StartupModule() override
	{
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/LidarPointCloud"), FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("LidarPointCloud"))->GetBaseDir(), TEXT("Shaders")));
	}
	virtual void ShutdownModule() override {}
	// End IModuleInterface Interface
};

IMPLEMENT_MODULE(FLidarPointCloudRuntimeModule, LidarPointCloudRuntime)

