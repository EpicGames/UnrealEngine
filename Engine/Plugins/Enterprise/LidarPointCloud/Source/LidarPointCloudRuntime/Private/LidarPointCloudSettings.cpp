// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudSettings.h"
#include "Math/UnrealMathUtility.h"

ULidarPointCloudSettings::ULidarPointCloudSettings()
	: DuplicateHandling(ELidarPointCloudDuplicateHandling::SelectBrighter)
	, MaxDistanceForDuplicate(KINDA_SMALL_NUMBER)
	, MaxBucketSize(200)
	, NodeGridResolution(96)
	, MultithreadingInsertionBatchSize(500000)
	, bUseAsyncImport(true)
	, bPrioritizeActiveViewport(true)
	, CachedNodeLifetime(5.0f)
	, bReleaseAssetAfterSaving(true)
	, bReleaseAssetAfterCooking(true)
	, bUseRenderDataSmoothing(true)
	, RenderDataSmoothingMaxFrametime(0.01f)
	, bUseFastRendering(true)
	, MeshingBatchSize(128)
	, bAutoCenterOnImport(true)
	, bAutoCalculateNormalsOnImport(false)
	, bAutoBuildCollisionOnImport(false)
	, ImportScale(100)
	, ExportScale(0.01f)
{
}
