// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "LidarPointCloudShared.h"
#include "LidarPointCloud.h"

double FBenchmarkTimer::Time = 0;

void FLidarPointCloudDataBuffer::MarkAsFree()
{
	if (PendingSize > 0)
	{
		Resize(PendingSize, true);
		PendingSize = 0;
	}

	bInUse = false;
}

void FLidarPointCloudDataBuffer::Initialize(const int32& Size)
{
	Data.AddUninitialized(Size);
}

void FLidarPointCloudDataBuffer::Resize(const int32& NewBufferSize, bool bForce /*= false*/)
{
	if (bInUse && !bForce)
	{
		// Don't want to resize while in use, flag the new size as pending
		// This will cause a resize as soon as the buffer is freed
		PendingSize = NewBufferSize;
	}
	else
	{
		int32 Delta = NewBufferSize - Data.Num();

		// Expand
		if (Delta > 0)
		{
			Data.AddUninitialized(Delta);
		}
		// Shrink
		else
		{
			Data.RemoveAtSwap(0, -Delta, EAllowShrinking::Yes);
		}
	}
}

FLidarPointCloudDataBufferManager::FLidarPointCloudDataBufferManager(const int32& BufferSize, const int32& MaxNumberOfBuffers)
	: BufferSize(BufferSize)
	, MaxNumberOfBuffers(MaxNumberOfBuffers)
	, NumBuffersCreated(1)
	, Head(FLidarPointCloudDataBuffer())
	, Tail(&Head)
{
	Head.Element.Initialize(BufferSize);
}

FLidarPointCloudDataBufferManager::~FLidarPointCloudDataBufferManager()
{
	TList<FLidarPointCloudDataBuffer>* Iterator = &Head;
	while (Iterator)
	{
		if (Iterator != &Head)
		{
			TList<FLidarPointCloudDataBuffer>* Tmp = Iterator;
			Iterator = Iterator->Next;
			delete Tmp;
		}
		else
		{
			Iterator = Iterator->Next;
		}
	}
}

FLidarPointCloudDataBuffer* FLidarPointCloudDataBufferManager::GetFreeBuffer()
{
	FLidarPointCloudDataBuffer* OutBuffer = nullptr;

	// Find available memory allocation
	do
	{
		TList<FLidarPointCloudDataBuffer>* Iterator = &Head;
		while (Iterator)
		{
			if (!Iterator->Element.bInUse)
			{
				OutBuffer = &Iterator->Element;
				break;
			}

			Iterator = Iterator->Next;
		}
	} while (!OutBuffer && MaxNumberOfBuffers > 0 && NumBuffersCreated >= MaxNumberOfBuffers);

	// If none found, add a new one
	if (!OutBuffer)
	{
		Tail->Next = new TList<FLidarPointCloudDataBuffer>(FLidarPointCloudDataBuffer());
		Tail = Tail->Next;
		OutBuffer = &Tail->Element;
		OutBuffer->Initialize(BufferSize);
		++NumBuffersCreated;
	}

	OutBuffer->bInUse = true;

	return OutBuffer;
}

void FLidarPointCloudDataBufferManager::Resize(const int32& NewBufferSize)
{
	// Skip, if no change required
	if (BufferSize == NewBufferSize)
	{
		return;
	}

	BufferSize = NewBufferSize;

	TList<FLidarPointCloudDataBuffer>* Iterator = &Head;
	while (Iterator)
	{
		Iterator->Element.Resize(NewBufferSize);
		Iterator = Iterator->Next;
	}
}

FLidarPointCloudClippingVolumeParams::FLidarPointCloudClippingVolumeParams(const ALidarClippingVolume* ClippingVolume)
	: Mode(ClippingVolume->Mode)
	, Priority(ClippingVolume->Priority)
	, Bounds(ClippingVolume->GetBounds().GetBox())
{
	const FVector Extent = ClippingVolume->GetActorScale3D() * 100;
	PackedShaderData = FMatrix( FPlane(ClippingVolume->GetActorLocation(), Mode == ELidarClippingVolumeMode::ClipInside),
								FPlane(ClippingVolume->GetActorForwardVector(), Extent.X),
								FPlane(ClippingVolume->GetActorRightVector(), Extent.Y),
								FPlane(ClippingVolume->GetActorUpVector(), Extent.Z));
}
