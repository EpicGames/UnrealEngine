// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Meshing/LidarPointCloudMeshing.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudOctree.h"
#include "LidarPointCloudOctreeMacros.h"
#include "Async/Async.h"
#include "Async/Future.h"

namespace
{
	FORCEINLINE uint64 CalculateNumPermutations(int32 NumPoints)
	{
		return NumPoints > 4000000 ? UINT64_MAX : NumPoints * (NumPoints - 1) * (NumPoints - 2) / 6;
	}

	namespace MarchingCubes
	{
		// Stores triangle indices
		constexpr int8 TriangleTable[256][16] =
		{
			{-1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 3, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 1, 9, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 8, 3, 9, 8, 1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 10, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 3, 1, 2, 10, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{9, 2, 10, 0, 2, 9, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{2, 8, 3, 2, 10, 8, 10, 9, 8, -1, -1, -1, -1, -1, -1, -1},
			{3, 11, 2, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 11, 2, 8, 11, 0, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 9, 0, 2, 3, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 11, 2, 1, 9, 11, 9, 8, 11, -1, -1, -1, -1, -1, -1, -1},
			{3, 10, 1, 11, 10, 3, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 10, 1, 0, 8, 10, 8, 11, 10, -1, -1, -1, -1, -1, -1, -1},
			{3, 9, 0, 3, 11, 9, 11, 10, 9, -1, -1, -1, -1, -1, -1, -1},
			{9, 8, 10, 10, 8, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 7, 8, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 3, 0, 7, 3, 4, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 1, 9, 8, 4, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 1, 9, 4, 7, 1, 7, 3, 1, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 10, 8, 4, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{3, 4, 7, 3, 0, 4, 1, 2, 10, -1, -1, -1, -1, -1, -1, -1},
			{9, 2, 10, 9, 0, 2, 8, 4, 7, -1, -1, -1, -1, -1, -1, -1},
			{2, 10, 9, 2, 9, 7, 2, 7, 3, 7, 9, 4, -1, -1, -1, -1},
			{8, 4, 7, 3, 11, 2, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{11, 4, 7, 11, 2, 4, 2, 0, 4, -1, -1, -1, -1, -1, -1, -1},
			{9, 0, 1, 8, 4, 7, 2, 3, 11, -1, -1, -1, -1, -1, -1, -1},
			{4, 7, 11, 9, 4, 11, 9, 11, 2, 9, 2, 1, -1, -1, -1, -1},
			{3, 10, 1, 3, 11, 10, 7, 8, 4, -1, -1, -1, -1, -1, -1, -1},
			{1, 11, 10, 1, 4, 11, 1, 0, 4, 7, 11, 4, -1, -1, -1, -1},
			{4, 7, 8, 9, 0, 11, 9, 11, 10, 11, 0, 3, -1, -1, -1, -1},
			{4, 7, 11, 4, 11, 9, 9, 11, 10, -1, -1, -1, -1, -1, -1, -1},
			{9, 5, 4, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{9, 5, 4, 0, 8, 3, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 5, 4, 1, 5, 0, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{8, 5, 4, 8, 3, 5, 3, 1, 5, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 10, 9, 5, 4, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{3, 0, 8, 1, 2, 10, 4, 9, 5, -1, -1, -1, -1, -1, -1, -1},
			{5, 2, 10, 5, 4, 2, 4, 0, 2, -1, -1, -1, -1, -1, -1, -1},
			{2, 10, 5, 3, 2, 5, 3, 5, 4, 3, 4, 8, -1, -1, -1, -1},
			{9, 5, 4, 2, 3, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 11, 2, 0, 8, 11, 4, 9, 5, -1, -1, -1, -1, -1, -1, -1},
			{0, 5, 4, 0, 1, 5, 2, 3, 11, -1, -1, -1, -1, -1, -1, -1},
			{2, 1, 5, 2, 5, 8, 2, 8, 11, 4, 8, 5, -1, -1, -1, -1},
			{10, 3, 11, 10, 1, 3, 9, 5, 4, -1, -1, -1, -1, -1, -1, -1},
			{4, 9, 5, 0, 8, 1, 8, 10, 1, 8, 11, 10, -1, -1, -1, -1},
			{5, 4, 0, 5, 0, 11, 5, 11, 10, 11, 0, 3, -1, -1, -1, -1},
			{5, 4, 8, 5, 8, 10, 10, 8, 11, -1, -1, -1, -1, -1, -1, -1},
			{9, 7, 8, 5, 7, 9, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{9, 3, 0, 9, 5, 3, 5, 7, 3, -1, -1, -1, -1, -1, -1, -1},
			{0, 7, 8, 0, 1, 7, 1, 5, 7, -1, -1, -1, -1, -1, -1, -1},
			{1, 5, 3, 3, 5, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{9, 7, 8, 9, 5, 7, 10, 1, 2, -1, -1, -1, -1, -1, -1, -1},
			{10, 1, 2, 9, 5, 0, 5, 3, 0, 5, 7, 3, -1, -1, -1, -1},
			{8, 0, 2, 8, 2, 5, 8, 5, 7, 10, 5, 2, -1, -1, -1, -1},
			{2, 10, 5, 2, 5, 3, 3, 5, 7, -1, -1, -1, -1, -1, -1, -1},
			{7, 9, 5, 7, 8, 9, 3, 11, 2, -1, -1, -1, -1, -1, -1, -1},
			{9, 5, 7, 9, 7, 2, 9, 2, 0, 2, 7, 11, -1, -1, -1, -1},
			{2, 3, 11, 0, 1, 8, 1, 7, 8, 1, 5, 7, -1, -1, -1, -1},
			{11, 2, 1, 11, 1, 7, 7, 1, 5, -1, -1, -1, -1, -1, -1, -1},
			{9, 5, 8, 8, 5, 7, 10, 1, 3, 10, 3, 11, -1, -1, -1, -1},
			{5, 7, 0, 5, 0, 9, 7, 11, 0, 1, 0, 10, 11, 10, 0, -1},
			{11, 10, 0, 11, 0, 3, 10, 5, 0, 8, 0, 7, 5, 7, 0, -1},
			{11, 10, 5, 7, 11, 5, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{10, 6, 5, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 3, 5, 10, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{9, 0, 1, 5, 10, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 8, 3, 1, 9, 8, 5, 10, 6, -1, -1, -1, -1, -1, -1, -1},
			{1, 6, 5, 2, 6, 1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 6, 5, 1, 2, 6, 3, 0, 8, -1, -1, -1, -1, -1, -1, -1},
			{9, 6, 5, 9, 0, 6, 0, 2, 6, -1, -1, -1, -1, -1, -1, -1},
			{5, 9, 8, 5, 8, 2, 5, 2, 6, 3, 2, 8, -1, -1, -1, -1},
			{2, 3, 11, 10, 6, 5, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{11, 0, 8, 11, 2, 0, 10, 6, 5, -1, -1, -1, -1, -1, -1, -1},
			{0, 1, 9, 2, 3, 11, 5, 10, 6, -1, -1, -1, -1, -1, -1, -1},
			{5, 10, 6, 1, 9, 2, 9, 11, 2, 9, 8, 11, -1, -1, -1, -1},
			{6, 3, 11, 6, 5, 3, 5, 1, 3, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 11, 0, 11, 5, 0, 5, 1, 5, 11, 6, -1, -1, -1, -1},
			{3, 11, 6, 0, 3, 6, 0, 6, 5, 0, 5, 9, -1, -1, -1, -1},
			{6, 5, 9, 6, 9, 11, 11, 9, 8, -1, -1, -1, -1, -1, -1, -1},
			{5, 10, 6, 4, 7, 8, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 3, 0, 4, 7, 3, 6, 5, 10, -1, -1, -1, -1, -1, -1, -1},
			{1, 9, 0, 5, 10, 6, 8, 4, 7, -1, -1, -1, -1, -1, -1, -1},
			{10, 6, 5, 1, 9, 7, 1, 7, 3, 7, 9, 4, -1, -1, -1, -1},
			{6, 1, 2, 6, 5, 1, 4, 7, 8, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 5, 5, 2, 6, 3, 0, 4, 3, 4, 7, -1, -1, -1, -1},
			{8, 4, 7, 9, 0, 5, 0, 6, 5, 0, 2, 6, -1, -1, -1, -1},
			{7, 3, 9, 7, 9, 4, 3, 2, 9, 5, 9, 6, 2, 6, 9, -1},
			{3, 11, 2, 7, 8, 4, 10, 6, 5, -1, -1, -1, -1, -1, -1, -1},
			{5, 10, 6, 4, 7, 2, 4, 2, 0, 2, 7, 11, -1, -1, -1, -1},
			{0, 1, 9, 4, 7, 8, 2, 3, 11, 5, 10, 6, -1, -1, -1, -1},
			{9, 2, 1, 9, 11, 2, 9, 4, 11, 7, 11, 4, 5, 10, 6, -1},
			{8, 4, 7, 3, 11, 5, 3, 5, 1, 5, 11, 6, -1, -1, -1, -1},
			{5, 1, 11, 5, 11, 6, 1, 0, 11, 7, 11, 4, 0, 4, 11, -1},
			{0, 5, 9, 0, 6, 5, 0, 3, 6, 11, 6, 3, 8, 4, 7, -1},
			{6, 5, 9, 6, 9, 11, 4, 7, 9, 7, 11, 9, -1, -1, -1, -1},
			{10, 4, 9, 6, 4, 10, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 10, 6, 4, 9, 10, 0, 8, 3, -1, -1, -1, -1, -1, -1, -1},
			{10, 0, 1, 10, 6, 0, 6, 4, 0, -1, -1, -1, -1, -1, -1, -1},
			{8, 3, 1, 8, 1, 6, 8, 6, 4, 6, 1, 10, -1, -1, -1, -1},
			{1, 4, 9, 1, 2, 4, 2, 6, 4, -1, -1, -1, -1, -1, -1, -1},
			{3, 0, 8, 1, 2, 9, 2, 4, 9, 2, 6, 4, -1, -1, -1, -1},
			{0, 2, 4, 4, 2, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{8, 3, 2, 8, 2, 4, 4, 2, 6, -1, -1, -1, -1, -1, -1, -1},
			{10, 4, 9, 10, 6, 4, 11, 2, 3, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 2, 2, 8, 11, 4, 9, 10, 4, 10, 6, -1, -1, -1, -1},
			{3, 11, 2, 0, 1, 6, 0, 6, 4, 6, 1, 10, -1, -1, -1, -1},
			{6, 4, 1, 6, 1, 10, 4, 8, 1, 2, 1, 11, 8, 11, 1, -1},
			{9, 6, 4, 9, 3, 6, 9, 1, 3, 11, 6, 3, -1, -1, -1, -1},
			{8, 11, 1, 8, 1, 0, 11, 6, 1, 9, 1, 4, 6, 4, 1, -1},
			{3, 11, 6, 3, 6, 0, 0, 6, 4, -1, -1, -1, -1, -1, -1, -1},
			{6, 4, 8, 11, 6, 8, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{7, 10, 6, 7, 8, 10, 8, 9, 10, -1, -1, -1, -1, -1, -1, -1},
			{0, 7, 3, 0, 10, 7, 0, 9, 10, 6, 7, 10, -1, -1, -1, -1},
			{10, 6, 7, 1, 10, 7, 1, 7, 8, 1, 8, 0, -1, -1, -1, -1},
			{10, 6, 7, 10, 7, 1, 1, 7, 3, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 6, 1, 6, 8, 1, 8, 9, 8, 6, 7, -1, -1, -1, -1},
			{2, 6, 9, 2, 9, 1, 6, 7, 9, 0, 9, 3, 7, 3, 9, -1},
			{7, 8, 0, 7, 0, 6, 6, 0, 2, -1, -1, -1, -1, -1, -1, -1},
			{7, 3, 2, 6, 7, 2, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{2, 3, 11, 10, 6, 8, 10, 8, 9, 8, 6, 7, -1, -1, -1, -1},
			{2, 0, 7, 2, 7, 11, 0, 9, 7, 6, 7, 10, 9, 10, 7, -1},
			{1, 8, 0, 1, 7, 8, 1, 10, 7, 6, 7, 10, 2, 3, 11, -1},
			{11, 2, 1, 11, 1, 7, 10, 6, 1, 6, 7, 1, -1, -1, -1, -1},
			{8, 9, 6, 8, 6, 7, 9, 1, 6, 11, 6, 3, 1, 3, 6, -1},
			{0, 9, 1, 11, 6, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{7, 8, 0, 7, 0, 6, 3, 11, 0, 11, 6, 0, -1, -1, -1, -1},
			{7, 11, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{7, 6, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{3, 0, 8, 11, 7, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 1, 9, 11, 7, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{8, 1, 9, 8, 3, 1, 11, 7, 6, -1, -1, -1, -1, -1, -1, -1},
			{10, 1, 2, 6, 11, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 10, 3, 0, 8, 6, 11, 7, -1, -1, -1, -1, -1, -1, -1},
			{2, 9, 0, 2, 10, 9, 6, 11, 7, -1, -1, -1, -1, -1, -1, -1},
			{6, 11, 7, 2, 10, 3, 10, 8, 3, 10, 9, 8, -1, -1, -1, -1},
			{7, 2, 3, 6, 2, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{7, 0, 8, 7, 6, 0, 6, 2, 0, -1, -1, -1, -1, -1, -1, -1},
			{2, 7, 6, 2, 3, 7, 0, 1, 9, -1, -1, -1, -1, -1, -1, -1},
			{1, 6, 2, 1, 8, 6, 1, 9, 8, 8, 7, 6, -1, -1, -1, -1},
			{10, 7, 6, 10, 1, 7, 1, 3, 7, -1, -1, -1, -1, -1, -1, -1},
			{10, 7, 6, 1, 7, 10, 1, 8, 7, 1, 0, 8, -1, -1, -1, -1},
			{0, 3, 7, 0, 7, 10, 0, 10, 9, 6, 10, 7, -1, -1, -1, -1},
			{7, 6, 10, 7, 10, 8, 8, 10, 9, -1, -1, -1, -1, -1, -1, -1},
			{6, 8, 4, 11, 8, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{3, 6, 11, 3, 0, 6, 0, 4, 6, -1, -1, -1, -1, -1, -1, -1},
			{8, 6, 11, 8, 4, 6, 9, 0, 1, -1, -1, -1, -1, -1, -1, -1},
			{9, 4, 6, 9, 6, 3, 9, 3, 1, 11, 3, 6, -1, -1, -1, -1},
			{6, 8, 4, 6, 11, 8, 2, 10, 1, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 10, 3, 0, 11, 0, 6, 11, 0, 4, 6, -1, -1, -1, -1},
			{4, 11, 8, 4, 6, 11, 0, 2, 9, 2, 10, 9, -1, -1, -1, -1},
			{10, 9, 3, 10, 3, 2, 9, 4, 3, 11, 3, 6, 4, 6, 3, -1},
			{8, 2, 3, 8, 4, 2, 4, 6, 2, -1, -1, -1, -1, -1, -1, -1},
			{0, 4, 2, 4, 6, 2, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 9, 0, 2, 3, 4, 2, 4, 6, 4, 3, 8, -1, -1, -1, -1},
			{1, 9, 4, 1, 4, 2, 2, 4, 6, -1, -1, -1, -1, -1, -1, -1},
			{8, 1, 3, 8, 6, 1, 8, 4, 6, 6, 10, 1, -1, -1, -1, -1},
			{10, 1, 0, 10, 0, 6, 6, 0, 4, -1, -1, -1, -1, -1, -1, -1},
			{4, 6, 3, 4, 3, 8, 6, 10, 3, 0, 3, 9, 10, 9, 3, -1},
			{10, 9, 4, 6, 10, 4, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 9, 5, 7, 6, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 3, 4, 9, 5, 11, 7, 6, -1, -1, -1, -1, -1, -1, -1},
			{5, 0, 1, 5, 4, 0, 7, 6, 11, -1, -1, -1, -1, -1, -1, -1},
			{11, 7, 6, 8, 3, 4, 3, 5, 4, 3, 1, 5, -1, -1, -1, -1},
			{9, 5, 4, 10, 1, 2, 7, 6, 11, -1, -1, -1, -1, -1, -1, -1},
			{6, 11, 7, 1, 2, 10, 0, 8, 3, 4, 9, 5, -1, -1, -1, -1},
			{7, 6, 11, 5, 4, 10, 4, 2, 10, 4, 0, 2, -1, -1, -1, -1},
			{3, 4, 8, 3, 5, 4, 3, 2, 5, 10, 5, 2, 11, 7, 6, -1},
			{7, 2, 3, 7, 6, 2, 5, 4, 9, -1, -1, -1, -1, -1, -1, -1},
			{9, 5, 4, 0, 8, 6, 0, 6, 2, 6, 8, 7, -1, -1, -1, -1},
			{3, 6, 2, 3, 7, 6, 1, 5, 0, 5, 4, 0, -1, -1, -1, -1},
			{6, 2, 8, 6, 8, 7, 2, 1, 8, 4, 8, 5, 1, 5, 8, -1},
			{9, 5, 4, 10, 1, 6, 1, 7, 6, 1, 3, 7, -1, -1, -1, -1},
			{1, 6, 10, 1, 7, 6, 1, 0, 7, 8, 7, 0, 9, 5, 4, -1},
			{4, 0, 10, 4, 10, 5, 0, 3, 10, 6, 10, 7, 3, 7, 10, -1},
			{7, 6, 10, 7, 10, 8, 5, 4, 10, 4, 8, 10, -1, -1, -1, -1},
			{6, 9, 5, 6, 11, 9, 11, 8, 9, -1, -1, -1, -1, -1, -1, -1},
			{3, 6, 11, 0, 6, 3, 0, 5, 6, 0, 9, 5, -1, -1, -1, -1},
			{0, 11, 8, 0, 5, 11, 0, 1, 5, 5, 6, 11, -1, -1, -1, -1},
			{6, 11, 3, 6, 3, 5, 5, 3, 1, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 10, 9, 5, 11, 9, 11, 8, 11, 5, 6, -1, -1, -1, -1},
			{0, 11, 3, 0, 6, 11, 0, 9, 6, 5, 6, 9, 1, 2, 10, -1},
			{11, 8, 5, 11, 5, 6, 8, 0, 5, 10, 5, 2, 0, 2, 5, -1},
			{6, 11, 3, 6, 3, 5, 2, 10, 3, 10, 5, 3, -1, -1, -1, -1},
			{5, 8, 9, 5, 2, 8, 5, 6, 2, 3, 8, 2, -1, -1, -1, -1},
			{9, 5, 6, 9, 6, 0, 0, 6, 2, -1, -1, -1, -1, -1, -1, -1},
			{1, 5, 8, 1, 8, 0, 5, 6, 8, 3, 8, 2, 6, 2, 8, -1},
			{1, 5, 6, 2, 1, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 3, 6, 1, 6, 10, 3, 8, 6, 5, 6, 9, 8, 9, 6, -1},
			{10, 1, 0, 10, 0, 6, 9, 5, 0, 5, 6, 0, -1, -1, -1, -1},
			{0, 3, 8, 5, 6, 10, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{10, 5, 6, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{11, 5, 10, 7, 5, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{11, 5, 10, 11, 7, 5, 8, 3, 0, -1, -1, -1, -1, -1, -1, -1},
			{5, 11, 7, 5, 10, 11, 1, 9, 0, -1, -1, -1, -1, -1, -1, -1},
			{10, 7, 5, 10, 11, 7, 9, 8, 1, 8, 3, 1, -1, -1, -1, -1},
			{11, 1, 2, 11, 7, 1, 7, 5, 1, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 3, 1, 2, 7, 1, 7, 5, 7, 2, 11, -1, -1, -1, -1},
			{9, 7, 5, 9, 2, 7, 9, 0, 2, 2, 11, 7, -1, -1, -1, -1},
			{7, 5, 2, 7, 2, 11, 5, 9, 2, 3, 2, 8, 9, 8, 2, -1},
			{2, 5, 10, 2, 3, 5, 3, 7, 5, -1, -1, -1, -1, -1, -1, -1},
			{8, 2, 0, 8, 5, 2, 8, 7, 5, 10, 2, 5, -1, -1, -1, -1},
			{9, 0, 1, 5, 10, 3, 5, 3, 7, 3, 10, 2, -1, -1, -1, -1},
			{9, 8, 2, 9, 2, 1, 8, 7, 2, 10, 2, 5, 7, 5, 2, -1},
			{1, 3, 5, 3, 7, 5, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 7, 0, 7, 1, 1, 7, 5, -1, -1, -1, -1, -1, -1, -1},
			{9, 0, 3, 9, 3, 5, 5, 3, 7, -1, -1, -1, -1, -1, -1, -1},
			{9, 8, 7, 5, 9, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{5, 8, 4, 5, 10, 8, 10, 11, 8, -1, -1, -1, -1, -1, -1, -1},
			{5, 0, 4, 5, 11, 0, 5, 10, 11, 11, 3, 0, -1, -1, -1, -1},
			{0, 1, 9, 8, 4, 10, 8, 10, 11, 10, 4, 5, -1, -1, -1, -1},
			{10, 11, 4, 10, 4, 5, 11, 3, 4, 9, 4, 1, 3, 1, 4, -1},
			{2, 5, 1, 2, 8, 5, 2, 11, 8, 4, 5, 8, -1, -1, -1, -1},
			{0, 4, 11, 0, 11, 3, 4, 5, 11, 2, 11, 1, 5, 1, 11, -1},
			{0, 2, 5, 0, 5, 9, 2, 11, 5, 4, 5, 8, 11, 8, 5, -1},
			{9, 4, 5, 2, 11, 3, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{2, 5, 10, 3, 5, 2, 3, 4, 5, 3, 8, 4, -1, -1, -1, -1},
			{5, 10, 2, 5, 2, 4, 4, 2, 0, -1, -1, -1, -1, -1, -1, -1},
			{3, 10, 2, 3, 5, 10, 3, 8, 5, 4, 5, 8, 0, 1, 9, -1},
			{5, 10, 2, 5, 2, 4, 1, 9, 2, 9, 4, 2, -1, -1, -1, -1},
			{8, 4, 5, 8, 5, 3, 3, 5, 1, -1, -1, -1, -1, -1, -1, -1},
			{0, 4, 5, 1, 0, 5, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{8, 4, 5, 8, 5, 3, 9, 0, 5, 0, 3, 5, -1, -1, -1, -1},
			{9, 4, 5, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 11, 7, 4, 9, 11, 9, 10, 11, -1, -1, -1, -1, -1, -1, -1},
			{0, 8, 3, 4, 9, 7, 9, 11, 7, 9, 10, 11, -1, -1, -1, -1},
			{1, 10, 11, 1, 11, 4, 1, 4, 0, 7, 4, 11, -1, -1, -1, -1},
			{3, 1, 4, 3, 4, 8, 1, 10, 4, 7, 4, 11, 10, 11, 4, -1},
			{4, 11, 7, 9, 11, 4, 9, 2, 11, 9, 1, 2, -1, -1, -1, -1},
			{9, 7, 4, 9, 11, 7, 9, 1, 11, 2, 11, 1, 0, 8, 3, -1},
			{11, 7, 4, 11, 4, 2, 2, 4, 0, -1, -1, -1, -1, -1, -1, -1},
			{11, 7, 4, 11, 4, 2, 8, 3, 4, 3, 2, 4, -1, -1, -1, -1},
			{2, 9, 10, 2, 7, 9, 2, 3, 7, 7, 4, 9, -1, -1, -1, -1},
			{9, 10, 7, 9, 7, 4, 10, 2, 7, 8, 7, 0, 2, 0, 7, -1},
			{3, 7, 10, 3, 10, 2, 7, 4, 10, 1, 10, 0, 4, 0, 10, -1},
			{1, 10, 2, 8, 7, 4, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 9, 1, 4, 1, 7, 7, 1, 3, -1, -1, -1, -1, -1, -1, -1},
			{4, 9, 1, 4, 1, 7, 0, 8, 1, 8, 7, 1, -1, -1, -1, -1},
			{4, 0, 3, 7, 4, 3, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{4, 8, 7, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{9, 10, 8, 10, 11, 8, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{3, 0, 9, 3, 9, 11, 11, 9, 10, -1, -1, -1, -1, -1, -1, -1},
			{0, 1, 10, 0, 10, 8, 8, 10, 11, -1, -1, -1, -1, -1, -1, -1},
			{3, 1, 10, 11, 3, 10, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 2, 11, 1, 11, 9, 9, 11, 8, -1, -1, -1, -1, -1, -1, -1},
			{3, 0, 9, 3, 9, 11, 1, 2, 9, 2, 11, 9, -1, -1, -1, -1},
			{0, 2, 11, 8, 0, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{3, 2, 11, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{2, 3, 8, 2, 8, 10, 10, 8, 9, -1, -1, -1, -1, -1, -1, -1},
			{9, 10, 2, 0, 9, 2, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{2, 3, 8, 2, 8, 10, 0, 1, 8, 1, 10, 8, -1, -1, -1, -1},
			{1, 10, 2, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{1, 3, 8, 9, 1, 8, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 9, 1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{0, 3, 8, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1},
			{-1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1, -1}
		};

		const FVector3f VertexOffsetTable[12] =
		{
			FVector3f(0.5, 0, 0),
			FVector3f(1, 0.5, 0),
			FVector3f(0.5, 1, 0),
			FVector3f(0, 0.5, 0),
			FVector3f(0.5, 0, 1),
			FVector3f(1, 0.5, 1),
			FVector3f(0.5, 1, 1),
			FVector3f(0, 0.5, 1),
			FVector3f(0, 0, 0.5),
			FVector3f(1, 0, 0.5),
			FVector3f(1, 1, 0.5),
			FVector3f(0, 1, 0.5)
		};

		// 256 possible marching cubes combinations
		constexpr int32 NumCombinations = 256;

		// Max 5 triangles x 3 vertices x 12 bytes each + 1 byte for NumVertices 
		constexpr int32 MaxCombinationSize = 181;

		/** Calculate pre-computed data, ready for processing. */
		TArray<uint8> BuildScaledVertexTable(float CellSize)
		{
			TArray<uint8> VertexData;
			VertexData.AddUninitialized(NumCombinations * MaxCombinationSize);
			
			TArray<FVector3f> TempArray;
			uint8* VertexDataPtr = VertexData.GetData();
			for (int32 i = 0; i < NumCombinations; ++i)
			{
				const int8* VertexList = TriangleTable[i];
				TempArray.Reset();

				for (int32 t = 0; t < 16; t += 3)
				{
					if (VertexList[t] == -1)
					{
						break;
					}

					TempArray.Add(VertexOffsetTable[VertexList[t + 2]] * CellSize);
					TempArray.Add(VertexOffsetTable[VertexList[t + 1]] * CellSize);
					TempArray.Add(VertexOffsetTable[VertexList[t]] * CellSize);
				}

				*VertexDataPtr++ = TempArray.Num();
				FMemory::Memcpy(VertexDataPtr, TempArray.GetData(), TempArray.Num() * 12);
				VertexDataPtr += MaxCombinationSize - 1;
			}

			return VertexData;
		}

		/** Constructs the normals table for vertices */
		TArray<TArray<FVector3f>>* GetNormalsTable()
		{
			static TArray<TArray<FVector3f>> NormalsTable;

			if(NormalsTable.Num() == 0)
			{
				NormalsTable.AddDefaulted(NumCombinations);
				for(int32 i = 0; i < NumCombinations; ++i)
				{
					TArray<FVector3f>& Normals = NormalsTable[i];
					Normals.Reserve(15);
					
					for(int32 j = 0; j < 15; j += 3)
					{
						const int8 A = TriangleTable[i][j];
						const int8 B = TriangleTable[i][j+1];
						const int8 C = TriangleTable[i][j+2];

						if(A > -1)
						{
							const FVector3f P1 = VertexOffsetTable[A];
							const FVector3f P2 = VertexOffsetTable[B];
							const FVector3f P3 = VertexOffsetTable[C];

							const FVector3f V1 = (P3 - P2).GetSafeNormal();
							const FVector3f V2 = (P1 - P2).GetSafeNormal();

							const FVector3f Normal = FVector3f::CrossProduct(V1, V2).GetSafeNormal();
							Normals.Append({Normal, Normal, Normal});
						}
						else
						{
							static TArray<FVector3f> EmptyNormal({ FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector });
							Normals.Append(EmptyNormal);
						}
					}
				}
			}

			return &NormalsTable;
		}

		/** Calculates the color lookup table for vertices */
		void CalculateColorLookupTable(int32 GridDimension, TArray<TArray<uint32>>& OutColorTable)
		{
			static TArray<TArray<FIntVector>> MasterColorOffsetTable;

			if(MasterColorOffsetTable.Num() == 0)
			{
				TArray<TArray<FVector3f>>* NormalsTable = GetNormalsTable();

				MasterColorOffsetTable.AddDefaulted(NumCombinations);
				for(int32 i = 0; i < NumCombinations; ++i)
				{
					TArray<FIntVector>& ColorOffsets = MasterColorOffsetTable[i];
					ColorOffsets.Reserve(15);
					
					for(int32 j = 0; j < 15; ++j)
					{
						const int8 Idx = TriangleTable[i][j];
						if(Idx > -1)
						{
							const static FVector3f Corners[8] =
							{
								FVector3f(0, 0, 0),
								FVector3f(1, 0, 0),
								FVector3f(0, 1, 0),
								FVector3f(1, 1, 0),
								FVector3f(0, 0, 1),
								FVector3f(1, 0, 1),
								FVector3f(0, 1, 1),
								FVector3f(1, 1, 1)
							};
							
							const FVector3f P = VertexOffsetTable[Idx];
							const FVector3f N = -(*NormalsTable)[i][j];

							float MinDist = FLT_MAX;
							int32 MinC = 0;

							for(int32 c = 0; c < 8; c++)
							{
								const FVector3f V = Corners[c] - P;
								if(FVector3f::DotProduct(N, V) > 0)
								{
									const float Dist = V.SizeSquared();
								
									if(Dist < MinDist)
									{
										MinDist = Dist;
										MinC = c;
									}
								}
							}

							ColorOffsets.Emplace((FVector)Corners[MinC]);
						}
					}
				}
			}

			const int32 GridDimensionSq = GridDimension * GridDimension;

			OutColorTable.SetNum(NumCombinations);
			for(int32 i = 0; i < NumCombinations; ++i)
			{
				TArray<FIntVector>& MasterOffsets = MasterColorOffsetTable[i];
				TArray<uint32>& ColorOffsets = OutColorTable[i];
				ColorOffsets.SetNumZeroed(MasterOffsets.Num());

				for(int32 j = 0; j < ColorOffsets.Num(); ++j)
				{
					const FIntVector& MasterOffset = MasterOffsets[j];
					ColorOffsets[j] = MasterOffset.Z * GridDimensionSq + MasterOffset.Y * GridDimension + MasterOffset.X;
				}
			}
		}

		/** Applies Marching Cubes algorithm to extract mesh information from the given voxelized grid */
		void ProcessGrid(uint8* VoxelizedGrid, uint8* VertexData, FColor* ColorGrid, TArray<TArray<uint32>>& ColorLookupTable, int32 GridDimension, float CellSize, FVector3f VertexOffset,
			TArray<FVector3f>& OutVertices, TArray<FVector3f>* OutNormals, TArray<FColor>* OutColors)
		{
			TArray<TArray<FVector3f>>* NormalsTable = GetNormalsTable();
			
			FVector3f Coords;
			uint8* Vertices = nullptr;

			const int32 GridDimensionSq = GridDimension * GridDimension;

			uint8* Grid = VoxelizedGrid;
			uint8* GridUp = Grid + GridDimensionSq;

			for (int32 sZ = 0; sZ < GridDimension - 1; ++sZ)
			{
				Coords.Z = VertexOffset.Z + (sZ + 0.5f) * CellSize;

				Grid = VoxelizedGrid + sZ * GridDimensionSq;
				GridUp = Grid + GridDimensionSq;
				int32 BaseIndex = 0;

				for (int32 sY = 0; sY < GridDimension - 1; ++sY)
				{
					Coords.Y = VertexOffset.Y + (sY + 0.5f) * CellSize;

					for (int32 sX = 0; sX < GridDimension - 1; ++sX, ++BaseIndex)
					{
						Coords.X = VertexOffset.X + (sX + 0.5f) * CellSize;

						// Calculate the variation of the cube
						uint8 CubeVariation = 0;
						if (Grid[BaseIndex] == 0) { CubeVariation |= 1; }
						if (Grid[BaseIndex + 1] == 0) { CubeVariation |= 2; }
						if (Grid[BaseIndex + GridDimension + 1] == 0) { CubeVariation |= 4; }
						if (Grid[BaseIndex + GridDimension] == 0) { CubeVariation |= 8; }
						if (GridUp[BaseIndex] == 0) { CubeVariation |= 16; }
						if (GridUp[BaseIndex + 1] == 0) { CubeVariation |= 32; }
						if (GridUp[BaseIndex + GridDimension + 1] == 0) { CubeVariation |= 64; }
						if (GridUp[BaseIndex + GridDimension] == 0) { CubeVariation |= 128; }

						// No triangles to add
						if (CubeVariation == 0 || CubeVariation == 255)
						{
							continue;
						}

						Vertices = &VertexData[CubeVariation * MaxCombinationSize];
						const uint8 NumVertices = Vertices[0];

						if (NumVertices == 0)
						{
							continue;
						}

						// Fill vertex positions
						{
							const int32 Offset = OutVertices.AddUninitialized(NumVertices);
							for (FVector3f* Vertex = OutVertices.GetData() + Offset, *SourceData = (FVector3f*)(Vertices + 1), *DataEnd = Vertex + NumVertices; Vertex != DataEnd; ++Vertex, ++SourceData)
							{
								*Vertex = Coords + *SourceData;
							}
						}

						// Fill normals if requested
						if(OutNormals)
						{
							const int32 Offset = OutNormals->AddUninitialized(NumVertices);
							FMemory::Memcpy(OutNormals->GetData() + Offset, (*NormalsTable)[CubeVariation].GetData(), NumVertices * sizeof(FVector3f));
						}

						// Fill colors if requested
						if(OutColors)
						{
							const uint32 BaseColorIndex = sZ * GridDimensionSq + BaseIndex;
							const uint32* LookupTablePtr = ColorLookupTable[CubeVariation].GetData();
							const int32 Offset = OutColors->AddUninitialized(NumVertices);
							FColor* ColorsPtr = OutColors->GetData() + Offset;
							for(int32 v = 0; v < NumVertices; ++v)
							{
								*ColorsPtr++ = ColorGrid[BaseColorIndex + *LookupTablePtr++];
							}
						}
					}

					++BaseIndex;
				}
			}
		}

		void Run(FLidarPointCloudOctree* Octree, const float& CellSize, bool bUseSelection, TArray<FVector3f>& OutVertices, TArray<FVector3f>* OutNormals, TArray<FColor>* OutColors)
		{
			FScopeBenchmarkTimer Timer("Marching Cubes");
			
			// Expand original bounds to make sure the mesh is closed at the edges
			const FBox Bounds = Octree->GetBounds().ExpandBy(FVector::OneVector * CellSize, FVector::OneVector * CellSize);
			const FVector3f BoundsSize = (FVector3f)Bounds.GetSize();
			const FVector3f LocationOffset = (FVector3f)Octree->GetOwner()->LocationOffset;

			// Number of cells in each axis
			const int32 BatchSize = GetDefault<ULidarPointCloudSettings>()->MeshingBatchSize;

			// Determines the number of samples to perform
			const FIntVector NumSamples((FVector)BoundsSize / ((BatchSize - 1) * CellSize) + 1);
			const int32 TotalNumSamples = NumSamples.X * NumSamples.Y * NumSamples.Z;

			// Precalculated for performance reasons
			const float InversedCellSize = 1 / CellSize;
			const FBox BaseSamplingBounds(Bounds.Min, Bounds.Min + BatchSize * CellSize);
			TArray<TArray<uint32>> ColorLookupTable;
			CalculateColorLookupTable(BatchSize, ColorLookupTable);

			// Multithreading
			const int32 MaxNumThreads = FMath::Min(FPlatformMisc::NumberOfCoresIncludingHyperthreads() - 1, TotalNumSamples);
			TArray<TFuture<void>> ThreadResults;
			FThreadSafeCounter SampleIndex = 0;

			// Data storage
			FCriticalSection VertexLock;
			OutVertices.Empty();

			TArray<uint8> VertexData = BuildScaledVertexTable(CellSize);
			uint8* VertexDataPtr = VertexData.GetData();
			
			// Make sure to acquire data release lock before loading nodes
			FScopeLock ReleaseLock(&Octree->DataReleaseLock);

			Octree->LoadAllNodes(false);

			const FLidarPointCloudOctree* OctreeConst = Octree;

			// Fire threads
			for (int32 t = 0; t < MaxNumThreads && SampleIndex.GetValue() < TotalNumSamples; t++)
			{
				ThreadResults.Add(Async(EAsyncExecution::TaskGraph, [&SampleIndex, bUseSelection, &OutVertices, &VertexLock, VertexDataPtr, BatchSize,
					OutNormals, OutColors, &ColorLookupTable, TotalNumSamples, NumSamples, BaseSamplingBounds, CellSize, OctreeConst, InversedCellSize, LocationOffset]
				{
					// Local caching arrays to reduce number of syncs required
					TArray<FVector3f> _Vertices;
					TArray<FVector3f> _Normals;
					TArray<FVector3f>* _NormalsPtr = OutNormals ? &_Normals : nullptr;
					TArray<FColor> _Colors;
					TArray<FColor>* _ColorsPtr = OutColors ? &_Colors : nullptr;
					
					const int32 NumCellsSq = BatchSize * BatchSize;
					const int32 NumCellsCu = NumCellsSq * BatchSize;

					uint8* VoxelizedGrid = new uint8[NumCellsCu];
					FColor* ColorGrid = nullptr;
					if(OutColors)
					{
						ColorGrid = new FColor[NumCellsCu];
					}

					const float OffsetMultiplier = (BatchSize - 1) * CellSize;

					TArray64<const FLidarPointCloudPoint*> Selection;

					int32 Index;
					while ((Index = SampleIndex.Add(1)) < TotalNumSamples)
					{
						const int32 W = Index % (NumSamples.X * NumSamples.Y);
						const FBox SamplingBounds = BaseSamplingBounds.ShiftBy(FVector(W % NumSamples.X, W / NumSamples.X, Index / (NumSamples.X * NumSamples.Y)) * OffsetMultiplier);

						// Sample the data using the calculated bounds
#if WITH_EDITOR
						if(bUseSelection)
						{
							OctreeConst->GetSelectedPointsInBox(Selection, SamplingBounds);
						}
						else
#endif
						{
							OctreeConst->GetPointsInBox(Selection, SamplingBounds, true);
						}
						
						// Skip if no points in selection
						if (Selection.Num() == 0)
						{
							continue;
						}

						// Clear existing grids
						FMemory::Memzero(VoxelizedGrid, NumCellsCu);
						if(ColorGrid)
						{
							FMemory::Memzero(ColorGrid, NumCellsCu * sizeof(FColor));
						}

						// Calculate voxelized grids
						for (const FLidarPointCloudPoint** Point = Selection.GetData(), **DataEnd = Selection.GetData() + Selection.Num(); Point != DataEnd; ++Point)
						{
							// Calculate location relative to sampling bounds
							const FVector3f Location = (*Point)->Location - (FVector3f)SamplingBounds.Min;

							// Calculate grid coordinates
							const FIntVector Grid(FMath::Min(BatchSize - 1, (int32)(Location.X * InversedCellSize)), FMath::Min(BatchSize - 1, (int32)(Location.Y * InversedCellSize)), FMath::Min(BatchSize - 1, (int32)(Location.Z * InversedCellSize)));

							const int32 GridIndex = Grid.Z * NumCellsSq + Grid.Y * BatchSize + Grid.X;

							VoxelizedGrid[GridIndex] = 1;

							if(ColorGrid)
							{
								ColorGrid[GridIndex] = (*Point)->Color;
							}
						}

						ProcessGrid(VoxelizedGrid, VertexDataPtr, ColorGrid, ColorLookupTable, BatchSize, CellSize, (FVector3f)SamplingBounds.Min + LocationOffset, _Vertices, _NormalsPtr, _ColorsPtr);
					}
				
					// Clean up
					delete[] VoxelizedGrid;
					if(ColorGrid)
					{
						delete[] ColorGrid;
					}

					// Sync
					FScopeLock Lock(&VertexLock);
					OutVertices.Append(_Vertices);
					if(OutNormals)
					{
						OutNormals->Append(_Normals);
					}
					if(OutColors)
					{
						OutColors->Append(_Colors);
					}
				}));
			}

			// Sync threads
			for (const TFuture<void>& ThreadResult : ThreadResults)
			{
				ThreadResult.Get();
			}

			Octree->ReleaseAllNodes(false);
		}
	}
}

void LidarPointCloudMeshing::FMeshBuffers::Init(uint32 Capacity, bool bExpandIfNotEmpty, uint32& OutIndexOffset, uint32& OutVertexOffset)
{
	if((Indices.IsEmpty() && Vertices.IsEmpty()) || !bExpandIfNotEmpty)
	{
		OutIndexOffset = OutVertexOffset = 0;
		Indices.SetNumUninitialized(Capacity);
		Vertices.Reserve(Capacity);
		Bounds = FBoxSphereBounds(EForceInit::ForceInit);
	}
	else
	{
		OutIndexOffset = Indices.AddUninitialized(Capacity);
		OutVertexOffset = Vertices.Num();
        Vertices.Reserve(Vertices.Num() + Capacity);
	}
}

void LidarPointCloudMeshing::FMeshBuffers::ExpandBounds(const FBox& NewBounds)
{
	const FBoxSphereBounds ConvertedNewBounds(NewBounds);
	Bounds = Bounds.BoxExtent == FVector::ZeroVector ? ConvertedNewBounds : Bounds + ConvertedNewBounds;
}

void LidarPointCloudMeshing::FMeshBuffers::NormalizeNormals()
{
	for (FVertexData *Vertex = Vertices.GetData(), *DataEnd = Vertex + Vertices.Num(); Vertex != DataEnd; ++Vertex)
	{
		Vertex->Normal.Normalize();
	}
}

void LidarPointCloudMeshing::CalculateNormals(FLidarPointCloudOctree* Octree, FThreadSafeBool* bCancelled, int32 Quality, float Tolerance, TArray64<FLidarPointCloudPoint*>& InPointSelection)
{
	/** Groups sampling information together for readability */
	struct FSamplingUnit
	{
		FVector3f Center;
		FVector3f Extent;
		TArray64<FLidarPointCloudPoint*> Points;
		FLidarPointCloudOctreeNode* Node;

		FSamplingUnit(const FVector3f& Center, const FVector3f& Extent, FLidarPointCloudOctreeNode* Node)
			: Center(Center)
			, Extent(Extent)
			, Node(Node)
		{
		}

		FSamplingUnit* ConstructChildAtLocation(int32 i)
		{
			return new FSamplingUnit(Center + Extent * (FVector3f(-0.5f) + FVector3f((i & 4) == 4, (i & 2) == 2, (i & 1) == 1)), Extent / 2, Node ? Node->GetChildNodeAtLocation(i) : nullptr);
		}
	};

	int32 DesiredNumIterations = Quality;

	const FLidarPointCloudNormal UpNormal = FVector3f::UpVector;

	TArray64<int32> Indices;

	TQueue<FSamplingUnit*> Q;
	{
		FSamplingUnit* Root = new FSamplingUnit(FVector3f::ZeroVector, Octree->SharedData[0].Extent, Octree->Root);
		if (InPointSelection.Num() == 0)
		{
			Octree->GetPoints(Root->Points);
		}
		else
		{
			Root->Points = InPointSelection;
		}
		Q.Enqueue(Root);
	}
	FSamplingUnit* SamplingUnit;
	while ((!bCancelled || !*bCancelled) && Q.Dequeue(SamplingUnit))
	{
		while (SamplingUnit->Points.Num() >= 3)
		{
			// Find Most Probable Plane
			FPlane BestPlane(EForceInit::ForceInit);
			{
				TArray64<FLidarPointCloudPoint*>* Points;
				bool bDestroyArray = false;

				// If the sampling unit is attached to an existing node, use its grid-allocated points to pick random models from - much more accurate and faster
				if (SamplingUnit->Node)
				{
					Points = new TArray64<FLidarPointCloudPoint*>();
					Points->Reserve(SamplingUnit->Node->GetNumPoints());
					FOR(Point, SamplingUnit->Node)
					{
						if (!Point->Normal.IsValid())
						{
							Points->Add(Point);
						}
					}

					// This is a temporary array - need to destroy it after it's used
					bDestroyArray = true;
				}
				// ... otherwise, just use whatever points are left
				else
				{
					Points = &SamplingUnit->Points;
				}

				TMap<FIntVector, uint32> PlaneModels;
				FIntVector CurrentModel;

				TArray64<FLidarPointCloudPoint*>& SelectedPoints = *Points;

				const int32 NumPoints = SelectedPoints.Num();
				const int32 MaxPointIndex = NumPoints - 1;
				const uint32 ConfidenceThreshold = NumPoints * 0.8f; // We are confident at 80% consensus
				const uint32 ValidThreshold = NumPoints / 2; // We need at least 50% for consensus

				Indices.Reset(Points->Num());
				for (int32 i = 0; i < NumPoints; ++i)
				{
					Indices.Add(i);
				}

				const int32 NumIterations = FMath::Min((uint64)DesiredNumIterations, CalculateNumPermutations(NumPoints));				
				for (int32 i = 0; i < NumIterations; ++i)
				{
					// Find Random Model
					do
					{
						int32 A = FMath::RandRange(0, MaxPointIndex);
						int32 B = FMath::RandRange(0, MaxPointIndex - 1);
						int32 C = FMath::RandRange(0, MaxPointIndex - 2);
						int32 X = Indices[A];
						Indices.RemoveAtSwap(A, EAllowShrinking::No);
						int32 Y = Indices[B];
						Indices.RemoveAtSwap(B, EAllowShrinking::No);
						int32 Z = Indices[C];
						Indices.Add(X);
						Indices.Add(Y);

						if (X > Y)
						{
							if (X > Z)
							{
								CurrentModel.X = X;
								CurrentModel.Y = Y > Z ? Y : Z;
								CurrentModel.Z = Y > Z ? Z : Y;
							}
							else
							{
								CurrentModel.X = Z;
								CurrentModel.Y = X;
								CurrentModel.Z = Y;
							}
						}
						else
						{
							if (Y > Z)
							{
								CurrentModel.X = Y;
								CurrentModel.Y = X > Z ? X : Z;
								CurrentModel.Z = X > Z ? Z : X;
							}
							else
							{
								CurrentModel.X = Z;
								CurrentModel.Y = Y;
								CurrentModel.Z = X;
							}
						}
					} while (PlaneModels.Find(CurrentModel));

					const FPlane Plane((FVector)SelectedPoints[CurrentModel.X]->Location, (FVector)SelectedPoints[CurrentModel.Y]->Location, (FVector)SelectedPoints[CurrentModel.Z]->Location);

					// Count Inner Points
					uint32 NumInnerPoints = 0;
					for (FLidarPointCloudPoint** Point = SelectedPoints.GetData(), **DataEnd = Point + NumPoints; Point != DataEnd; ++Point)
					{
						if (FMath::Abs(Plane.PlaneDot((FVector)(*Point)->Location)) <= Tolerance)
						{
							++NumInnerPoints;
						}
					}

					// Confidence is high enough, we can stop here
					if (NumInnerPoints >= ConfidenceThreshold)
					{
						BestPlane = Plane;
						break;
					}

					PlaneModels.Emplace(CurrentModel, NumInnerPoints);
				}

				// If the best plane has not been found yet, pick the highest scoring one
				if (BestPlane.W == 0)
				{
					CurrentModel = FIntVector::NoneValue;

					// Anything with less points then the Valid Threshold will not be considered
					uint32 NumInnerPoints = ValidThreshold;
					for (TPair<FIntVector, uint32>& Model : PlaneModels)
					{
						if (Model.Value > NumInnerPoints)
						{
							CurrentModel = Model.Key;
							NumInnerPoints = Model.Value;
						}
					}

					BestPlane = CurrentModel != FIntVector::NoneValue ? FPlane((FVector)SelectedPoints[CurrentModel.X]->Location, (FVector)SelectedPoints[CurrentModel.Y]->Location, (FVector)SelectedPoints[CurrentModel.Z]->Location) : FPlane(EForceInit::ForceInit);
				}

				// If the points array was created temporarily, destroy it
				if (bDestroyArray)
				{
					delete Points;
					Points = nullptr;
				}
			}

			bool bSuccess = false;

			if (BestPlane.W != 0)
			{
				const FLidarPointCloudNormal Normal = BestPlane;

				int32 PointCount = SamplingUnit->Points.Num();

				// Apply Normals
				for (FLidarPointCloudPoint** DataStart = SamplingUnit->Points.GetData(), **Point = DataStart, **DataEnd = Point + SamplingUnit->Points.Num(); Point != DataEnd; ++Point)
				{
					if (FMath::Abs(BestPlane.PlaneDot((FVector)(*Point)->Location)) <= Tolerance)
					{
						(*Point)->Normal = Normal;
						--DataEnd;
						SamplingUnit->Points.RemoveAtSwap(Point-- - DataStart, EAllowShrinking::No);
					}
				}

				bSuccess = PointCount - SamplingUnit->Points.Num() > 0;
			}

			if(!bSuccess)
			{
				FSamplingUnit* Sublevels[8];
				for (int32 i = 0; i < 8; ++i)
				{
					Sublevels[i] = SamplingUnit->ConstructChildAtLocation(i);
				}

				// Build and enqueue sub-levels
				for (FLidarPointCloudPoint** Point = SamplingUnit->Points.GetData(), **DataEnd = Point + SamplingUnit->Points.Num(); Point != DataEnd; ++Point)
				{
					Sublevels[((*Point)->Location.X > SamplingUnit->Center.X ? 4 : 0) + ((*Point)->Location.Y > SamplingUnit->Center.Y ? 2 : 0) + ((*Point)->Location.Z > SamplingUnit->Center.Z)]->Points.Add(*Point);
				}

				SamplingUnit->Points.Empty();

				// Enqueue
				for (int32 i = 0; i < 8; ++i)
				{
					if (Sublevels[i]->Points.Num() > 0)
					{
						Q.Enqueue(Sublevels[i]);
					}
					else
					{
						delete Sublevels[i];
					}
				}
			}
		}

		// To any stray points left simply apply Up Vector
		for (FLidarPointCloudPoint* P : SamplingUnit->Points)
		{
			P->Normal = UpNormal;
		}

		// Delete when finished processing
		delete SamplingUnit;
		SamplingUnit = nullptr;
	}
}

void LidarPointCloudMeshing::BuildCollisionMesh(FLidarPointCloudOctree* Octree, const float& CellSize, FTriMeshCollisionData* CollisionMesh)
{
	if (!CollisionMesh)
	{
		return;
	}
	
	TArray<FVector3f> Vertices;
	MarchingCubes::Run(Octree, CellSize, false, Vertices, nullptr, nullptr);

	FScopeBenchmarkTimer Timer("Exporting Collision Data");
	
	CollisionMesh->Vertices.Empty(Vertices.Num());
	CollisionMesh->Indices.Empty(Vertices.Num() / 3);
	CollisionMesh->Indices.AddUninitialized(Vertices.Num() / 3);

	bool bInSet = false;
	TSet<FVector3f> TmpVertexData;
	TmpVertexData.Reserve(Vertices.Num());
	uint32* IndicesPtr = (uint32*)CollisionMesh->Indices.GetData();
	for (FVector3f *Vertex = Vertices.GetData(), *DataEnd = Vertex + Vertices.Num(); Vertex != DataEnd; ++Vertex, ++IndicesPtr)
	{
		*IndicesPtr = TmpVertexData.Add(*Vertex, &bInSet).AsInteger();
		if (!bInSet)
		{
			CollisionMesh->Vertices.Add(*Vertex);
		}
	}

	CollisionMesh->Vertices.Shrink();
}

void LidarPointCloudMeshing::BuildStaticMeshBuffers(FLidarPointCloudOctree* Octree, const float& CellSize, bool bUseSelection, FMeshBuffers* OutMeshBuffers, const FTransform& Transform)
{
	if(OutMeshBuffers == nullptr)
	{
		return;
	}
	
	TArray<FVector3f> Vertices;
	TArray<FVector3f> Normals;
	TArray<FColor> Colors;
	MarchingCubes::Run(Octree, CellSize, bUseSelection, Vertices, &Normals, &Colors);
	
	FScopeBenchmarkTimer Timer("Building Static Mesh Buffers");

	FBox Bounds(EForceInit::ForceInit);
	uint32 IndexOffset;
	uint32 VertexOffset;
	OutMeshBuffers->Init(Vertices.Num(), true, IndexOffset, VertexOffset);

	// We do not support doubles yet
	const FTransform3f Transform3f = (FTransform3f)Transform;

	bool bInSet = false;
	TSet<FIntVector3> TmpVertexData;
	TmpVertexData.Reserve(Vertices.Num());
	uint32* IndicesPtr = OutMeshBuffers->Indices.GetData() + IndexOffset;
	FVector3f* NormalsPtr = Normals.GetData();
	FColor* ColorsPtr = Colors.GetData();
	for (FVector3f *Vertex = Vertices.GetData(), *DataEnd = Vertex + Vertices.Num(); Vertex != DataEnd; ++Vertex, ++IndicesPtr, ++NormalsPtr, ++ColorsPtr)
	{
		*IndicesPtr = TmpVertexData.Add(FIntVector3((FVector)(*Vertex * 100)), &bInSet).AsInteger() + VertexOffset;
		if (bInSet)
		{
			OutMeshBuffers->Vertices[*IndicesPtr].Normal += *NormalsPtr;
		}
		else
		{
			const FVector3f Location = Transform3f.TransformPosition(*Vertex);
			OutMeshBuffers->Vertices.Emplace(Location, *NormalsPtr, *ColorsPtr);
			Bounds += (FVector)Location;
		}
	};
	
	OutMeshBuffers->ExpandBounds(Bounds);
	OutMeshBuffers->NormalizeNormals();
}
