// Copyright Epic Games, Inc. All Rights Reserved.

#include "LidarPointCloudRenderBuffers.h"
#include "RenderResource.h"
#include "VertexFactory.h"
#include "MeshBatch.h"
#include "RenderCommandFence.h"
#include "LidarPointCloudOctree.h"
#include "LidarPointCloudSettings.h"
#include "MaterialDomain.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"

#if WITH_EDITOR
#include "Settings/EditorStyleSettings.h"
#endif

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLidarPointCloudVertexFactoryUniformShaderParameters, "LidarVF");

#define BINDPARAM(Name) Name.Bind(ParameterMap, TEXT(#Name))
#define SETPARAM(Name) if (Name.IsBound()) { ShaderBindings.Add(Name, UserData->Name); }
#define SETSRVPARAM(Name) if(UserData->Name) { SETPARAM(Name) }

//////////////////////////////////////////////////////////// Base Buffer

TGlobalResource<FLidarPointCloudIndexBuffer> GLidarPointCloudIndexBuffer;
TGlobalResource<FLidarPointCloudSharedVertexFactory> GLidarPointCloudSharedVertexFactory;
TGlobalResource<FLidarPointCloudRenderBuffer> GDummyLidarPointCloudRenderBuffer(4);

//////////////////////////////////////////////////////////// Index Buffer

void FLidarPointCloudIndexBuffer::Resize(const uint32 & RequestedCapacity)
{
	// This must be called from Rendering thread
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	if (Capacity != RequestedCapacity)
	{
		ReleaseResource();
		Capacity = RequestedCapacity;
		InitResource(RHICmdList);
	}
}

void FLidarPointCloudIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	PointOffset = Capacity * 6;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateIndex<uint32>(TEXT("FLidarPointCloudIndexBuffer"), Capacity * 7)
		.AddUsage(EBufferUsageFlags::Dynamic)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<uint32> Data = RHICmdList.CreateBufferInitializer(CreateDesc);

	for (uint32 i = 0, idx = 0; i < Capacity; i++)
	{
		const uint32 v = i * 4;

		// Full quads
		Data[idx++] = v;
		Data[idx++] = v + 1;
		Data[idx++] = v + 2;
		Data[idx++] = v;
		Data[idx++] = v + 2;
		Data[idx++] = v + 3;

		// Points
		Data[PointOffset + i] = v;
	}

	IndexBufferRHI = Data.Finalize();
}

//////////////////////////////////////////////////////////// Structured Buffer

void FLidarPointCloudRenderBuffer::Resize(const uint32& RequestedCapacity)
{
	// This must be called from Rendering thread
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	if (Capacity != RequestedCapacity)
	{
		ReleaseResource();
		Capacity = RequestedCapacity;
		InitResource(RHICmdList);
	}
	else if (!IsInitialized())
	{
		InitResource(RHICmdList);
	}
}

void FLidarPointCloudRenderBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("FLidarPointCloudRenderBuffer"), Capacity)
		.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic)
		.DetermineInitialState();

	Buffer = RHICmdList.CreateBuffer(CreateDesc);
	SRV = RHICmdList.CreateShaderResourceView(
		Buffer,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_FLOAT));

	FLidarPointCloudVertexFactoryUniformShaderParameters UniformParameters;
	UniformParameters.VertexFetch_Buffer = SRV;
	UniformBuffer = TUniformBufferRef<FLidarPointCloudVertexFactoryUniformShaderParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_MultiFrame);
}

void FLidarPointCloudRenderBuffer::ReleaseRHI()
{
	// This must be called from Rendering thread
	check(IsInRenderingThread());

	if(UniformBuffer.IsValid())
	{
		UniformBuffer.SafeRelease();
	}
	
	if (Buffer)
	{
		Buffer.SafeRelease();
	}

	SRV.SafeRelease();
}

void FLidarPointCloudRenderBuffer::Initialize(FLidarPointCloudPoint* Data, int32 NumPoints)
{
	Resize(NumPoints * 5);

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	uint8* StructuredBuffer = (uint8*)RHICmdList.LockBuffer(Buffer, 0, NumPoints * sizeof(FLidarPointCloudPoint), RLM_WriteOnly);
	for (FLidarPointCloudPoint* P = Data, *DataEnd = P + NumPoints; P != DataEnd; ++P)
	{
		FMemory::Memcpy(StructuredBuffer, P, sizeof(FLidarPointCloudPoint));
		StructuredBuffer += sizeof(FLidarPointCloudPoint);
	}
	RHICmdList.UnlockBuffer(Buffer);
}

//////////////////////////////////////////////////////////// Ray Tracing Geometry

void FLidarPointCloudRayTracingGeometry::Initialize(int32 NumPoints)
{
#if RHI_RAYTRACING
	if (IsInitialized())
	{
		ReleaseResource();
	}
	
	NumPrimitives = NumPoints * 2;
	NumVertices = NumPoints * 4;

	FRayTracingGeometryInitializer NewInitializer;
	NewInitializer.IndexBuffer = GLidarPointCloudIndexBuffer.IndexBufferRHI;
	NewInitializer.TotalPrimitiveCount = NumPrimitives;
	NewInitializer.GeometryType = RTGT_Triangles;
	NewInitializer.bFastBuild = true;
	NewInitializer.bAllowUpdate = true;

	FRayTracingGeometrySegment Segment;
	Segment.VertexBuffer = nullptr;
	Segment.NumPrimitives = NumPrimitives;
	Segment.MaxVertices = NumVertices;
	NewInitializer.Segments.Add(Segment);

	SetInitializer(MoveTemp(NewInitializer));

	InitResource(FRHICommandListImmediate::Get());
#endif
}

//////////////////////////////////////////////////////////// User Data

FLidarPointCloudBatchElementUserData::FLidarPointCloudBatchElementUserData()
	: SelectionColor(FVector::OneVector)
	, NumClippingVolumes(0)
	, bStartClipped(false)
{
	for (int32 i = 0; i < 16; ++i)
	{
		ClippingVolume[i] = FMatrix44f(FPlane4f(FVector3f::ZeroVector, 0),
									FPlane4f(FVector3f::ForwardVector, FLT_MAX),
									FPlane4f(FVector3f::RightVector, FLT_MAX),
									FPlane4f(FVector3f::UpVector, FLT_MAX));
	}

#if WITH_EDITOR
	SelectionColor = FVector3f(GetDefault<UEditorStyleSettings>()->SelectionColor.ToFColor(true));
#endif
}

void FLidarPointCloudBatchElementUserData::SetClassificationColors(const TMap<int32, FLinearColor>& InClassificationColors)
{
	for (int32 i = 0; i < 32; ++i)
	{
		const FLinearColor* Color = InClassificationColors.Find(i);
		ClassificationColors[i] = Color ? FVector4f(*Color) : FVector4f(1, 1, 1);
	}
}

//////////////////////////////////////////////////////////// Vertex Factory

void FLidarPointCloudVertexFactoryShaderParameters::Bind(const FShaderParameterMap& ParameterMap)
{
	BINDPARAM(TreeBuffer);
	BINDPARAM(bEditorView);
	BINDPARAM(SelectionColor);
	BINDPARAM(LocationOffset);
	BINDPARAM(RootCellSize);
	BINDPARAM(RootExtent);
	BINDPARAM(bUsePerPointScaling);
	BINDPARAM(VirtualDepth);
	BINDPARAM(SpriteSizeMultiplier);
	BINDPARAM(ReversedVirtualDepthMultiplier);
	BINDPARAM(ViewRightVector);
	BINDPARAM(ViewUpVector);
	BINDPARAM(bUseCameraFacing);
	BINDPARAM(bUseScreenSizeScaling);
	BINDPARAM(bUseStaticBuffers);
	BINDPARAM(BoundsSize);
	BINDPARAM(ElevationColorBottom);
	BINDPARAM(ElevationColorTop);
	BINDPARAM(bUseCircle);
	BINDPARAM(bUseColorOverride);
	BINDPARAM(bUseElevationColor);
	BINDPARAM(Offset);
	BINDPARAM(Contrast);
	BINDPARAM(Saturation);
	BINDPARAM(Gamma);
	BINDPARAM(Tint);
	BINDPARAM(IntensityInfluence);
	BINDPARAM(bUseClassification);
	BINDPARAM(bUseClassificationAlpha);
	BINDPARAM(ClassificationColors);
	BINDPARAM(ClippingVolume);
	BINDPARAM(NumClippingVolumes);
	BINDPARAM(bStartClipped);
}

void FLidarPointCloudVertexFactoryShaderParameters::GetElementShaderBindings(const class FSceneInterface* Scene, const FSceneView* View, const FMeshMaterialShader* Shader, const EVertexInputStreamType InputStreamType, ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory, const FMeshBatchElement& BatchElement, class FMeshDrawSingleShaderBindings& ShaderBindings, FVertexInputStreamArray& VertexStreams) const
{
	FRHIUniformBuffer* VertexFactoryUniformBuffer = static_cast<FRHIUniformBuffer*>(BatchElement.VertexFactoryUserData);
	FLidarPointCloudBatchElementUserData* UserData = (FLidarPointCloudBatchElementUserData*)BatchElement.UserData;

	SETSRVPARAM(TreeBuffer);
	
	if(VertexFactoryUniformBuffer)
	{
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLidarPointCloudVertexFactoryUniformShaderParameters>(), VertexFactoryUniformBuffer);
	}
	
	SETPARAM(bEditorView);
	SETPARAM(SelectionColor);
	SETPARAM(LocationOffset);
	SETPARAM(RootCellSize);
	SETPARAM(RootExtent);
	SETPARAM(bUsePerPointScaling);
	SETPARAM(VirtualDepth);
	SETPARAM(SpriteSizeMultiplier);
	SETPARAM(ReversedVirtualDepthMultiplier);
	SETPARAM(ViewRightVector);
	SETPARAM(ViewUpVector);
	SETPARAM(bUseCameraFacing);
	SETPARAM(bUseScreenSizeScaling);
	SETPARAM(bUseStaticBuffers);
	SETPARAM(BoundsSize);
	SETPARAM(ElevationColorBottom);
	SETPARAM(ElevationColorTop);
	SETPARAM(bUseCircle);
	SETPARAM(bUseColorOverride);
	SETPARAM(bUseElevationColor);
	SETPARAM(Offset);
	SETPARAM(Contrast);
	SETPARAM(Saturation);
	SETPARAM(Gamma);
	SETPARAM(Tint);
	SETPARAM(IntensityInfluence);
	SETPARAM(bUseClassification);
	SETPARAM(bUseClassificationAlpha);
	SETPARAM(ClassificationColors);
	SETPARAM(ClippingVolume);
	SETPARAM(NumClippingVolumes);
	SETPARAM(bStartClipped);
}

bool FLidarPointCloudVertexFactoryBase::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return (IsPCPlatform(Parameters.Platform) && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
		Parameters.MaterialParameters.MaterialDomain == MD_Surface && Parameters.MaterialParameters.bIsUsedWithLidarPointCloud) || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

void FLidarPointCloudVertexFactoryBase::ModifyCompilationEnvironment(
	const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_LOCAL_SPACE"), TEXT("1"));
}

FShaderResourceViewRHIRef FLidarPointCloudVertexFactoryBase::GetVertexBufferSRV()
{
	return GDummyLidarPointCloudRenderBuffer.SRV;
}

void FLidarPointCloudVertexFactoryBase::InitRHI(FRHICommandListBase& RHICmdList)
{
	FLidarPointCloudVertexFactoryUniformShaderParameters UniformParameters;
	UniformParameters.VertexFetch_Buffer = GetVertexBufferSRV();
	if(UniformParameters.VertexFetch_Buffer)
	{
		UniformBuffer = TUniformBufferRef<FLidarPointCloudVertexFactoryUniformShaderParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_MultiFrame);
	}
}

void FLidarPointCloudVertexFactoryBase::ReleaseRHI()
{
	UniformBuffer.SafeRelease();
	FVertexFactory::ReleaseRHI();
}

void FLidarPointCloudVertexFactory::Initialize(FLidarPointCloudPoint* Data, int32 NumPoints)
{
	if (IsInitialized())
	{
		ReleaseResource();
	}

	VertexBuffer.Data = Data;
	VertexBuffer.NumPoints = NumPoints;

	InitResource(FRHICommandListImmediate::Get());
}

void FLidarPointCloudVertexFactory::FPointCloudVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const uint32 BufferSize = NumPoints * 4 * sizeof(FLidarPointCloudPoint);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("FPointCloudVertexBuffer"), BufferSize, sizeof(FLidarPointCloudPoint), EBufferUsageFlags::VertexBuffer)
		.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<FLidarPointCloudPoint> InitialData = RHICmdList.CreateBufferInitializer(CreateDesc);

	FLidarPointCloudPoint* Dest = InitialData.GetWritableData();
	for (int32 i = 0; i < NumPoints; ++i, ++Data)
	{
		FMemory::Memcpy(Dest, Data, sizeof(FLidarPointCloudPoint)); Dest++;
		FMemory::Memcpy(Dest, Data, sizeof(FLidarPointCloudPoint)); Dest++;
		FMemory::Memcpy(Dest, Data, sizeof(FLidarPointCloudPoint)); Dest++;
		FMemory::Memcpy(Dest, Data, sizeof(FLidarPointCloudPoint)); Dest++;
	}

	VertexBufferRHI = InitialData.Finalize();
	VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_FLOAT));
}

void FLidarPointCloudVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	VertexBuffer.InitResource(RHICmdList);

	FVertexDeclarationElementList Elements;
	Elements.Add(AccessStreamComponent(FVertexStreamComponent(&VertexBuffer, 0, sizeof(FLidarPointCloudPoint), VET_Float3), 0));
	Elements.Add(AccessStreamComponent(FVertexStreamComponent(&VertexBuffer, 12, sizeof(FLidarPointCloudPoint), VET_Color), 1));
	Elements.Add(AccessStreamComponent(FVertexStreamComponent(&VertexBuffer, 16, sizeof(FLidarPointCloudPoint), VET_UInt), 2));
	InitDeclaration(Elements);

	FLidarPointCloudVertexFactoryBase::InitRHI(RHICmdList);
}

void FLidarPointCloudVertexFactory::ReleaseRHI()
{
	FLidarPointCloudVertexFactoryBase::ReleaseRHI();
	VertexBuffer.ReleaseResource();
}

void FLidarPointCloudSharedVertexFactory::FPointCloudVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FPointCloudVertexBuffer"), sizeof(FVector))
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionZeroData();

	VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
}

void FLidarPointCloudSharedVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	VertexBuffer.InitResource(RHICmdList);

	FVertexDeclarationElementList Elements;
	Elements.Add(AccessStreamComponent(FVertexStreamComponent(&VertexBuffer, 0, 0, VET_Float3), 0));
	Elements.Add(AccessStreamComponent(FVertexStreamComponent(&VertexBuffer, 0, 0, VET_Color), 1));
	Elements.Add(AccessStreamComponent(FVertexStreamComponent(&VertexBuffer, 0, 0, VET_Color), 2));
	InitDeclaration(Elements);

	FLidarPointCloudVertexFactoryBase::InitRHI(RHICmdList);
}

void FLidarPointCloudSharedVertexFactory::ReleaseRHI()
{
	FLidarPointCloudVertexFactoryBase::ReleaseRHI();
	VertexBuffer.ReleaseResource();
}

IMPLEMENT_TYPE_LAYOUT(FLidarPointCloudVertexFactoryShaderParameters);

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLidarPointCloudVertexFactoryBase, SF_Vertex, FLidarPointCloudVertexFactoryShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLidarPointCloudVertexFactoryBase, SF_Compute, FLidarPointCloudVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLidarPointCloudVertexFactoryBase, SF_RayHitGroup, FLidarPointCloudVertexFactoryShaderParameters);
#endif

IMPLEMENT_VERTEX_FACTORY_TYPE(FLidarPointCloudVertexFactoryBase, "/Plugin/LidarPointCloud/Private/LidarPointCloudVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPositionOnly
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
);

#undef BINDPARAM
#undef SETPARAM
#undef SETSRVPARAM
