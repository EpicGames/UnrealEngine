// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RayTracingGeometry.h"
#include "VertexFactory.h"
#include "LidarPointCloudShared.h"
#include "HAL/ThreadSafeBool.h"
#include "MeshMaterialShader.h"

class FLidarPointCloudVertexFactory;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLidarPointCloudVertexFactoryUniformShaderParameters,)
	SHADER_PARAMETER_SRV(Buffer<float>, VertexFetch_Buffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/**
 * Base class for the buffers
 */
class FLidarPointCloudBuffer
{
public:
	FLidarPointCloudBuffer() : FLidarPointCloudBuffer(100000) {}
	FLidarPointCloudBuffer(uint32 Capacity) : Capacity(Capacity) {}

	/** Resizes the buffer to the specified capacity, if necessary. Must be called from Rendering thread. */
	virtual void Resize(const uint32& RequestedCapacity) = 0;
	
	FORCEINLINE uint32 GetCapacity() const { return Capacity; }

protected:
	uint32 Capacity;
};

/**
 * This class creates an IndexBuffer shared between all assets and all instances.
 */
class FLidarPointCloudIndexBuffer : public FLidarPointCloudBuffer, public FIndexBuffer
{
public:
	FLidarPointCloudIndexBuffer() : FLidarPointCloudBuffer(100000) {}

	uint32 PointOffset;

	virtual void Resize(const uint32& RequestedCapacity) override;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;	
};

/**
 * Encapsulates a GPU read buffer with its SRV.
 */
class FLidarPointCloudRenderBuffer : public FLidarPointCloudBuffer, public FRenderResource
{
public:
	FLidarPointCloudRenderBuffer() : FLidarPointCloudBuffer() {}
	FLidarPointCloudRenderBuffer(uint32 Capacity) : FLidarPointCloudBuffer(Capacity) {}

	void Initialize(FLidarPointCloudPoint* Data, int32 NumPoints);
	
	virtual FRHIUniformBuffer* GetUniformBuffer() const { return UniformBuffer.GetReference(); }

	FBufferRHIRef Buffer;
	FShaderResourceViewRHIRef SRV;

	int32 PointCount;

	virtual void Resize(const uint32& RequestedCapacity) override;	
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

private:
	TUniformBufferRef<FLidarPointCloudVertexFactoryUniformShaderParameters> UniformBuffer;
};

class FLidarPointCloudRayTracingGeometry : public FRayTracingGeometry
{
public:
	FLidarPointCloudRayTracingGeometry() {}
	virtual ~FLidarPointCloudRayTracingGeometry() {}
	
	void Initialize(int32 NumPoints);

	uint32 GetNumVertices() const { return NumVertices; }
	uint32 GetNumPrimitives() const { return NumPrimitives; }
	uint32 GetBufferSize() const { return NumVertices * sizeof(FLidarPointCloudPoint); }

private:
	uint32 NumVertices;
	uint32 NumPrimitives;
};

/**
 * Holds all data to be passed to the FLidarPointCloudVertexFactoryShaderParameters as UserData
 */
struct FLidarPointCloudBatchElementUserData
{
	FRHIShaderResourceView* TreeBuffer;
	int32 bEditorView;
	FVector3f SelectionColor;
	FVector3f LocationOffset;
	float RootCellSize;
	FVector3f RootExtent;
	int32 bUsePerPointScaling;
	float VirtualDepth;
	float SpriteSizeMultiplier;
	float ReversedVirtualDepthMultiplier;
	FVector3f ViewRightVector;
	FVector3f ViewUpVector;
	int32 bUseCameraFacing;
	int32 bUseScreenSizeScaling;
	int32 bUseStaticBuffers;
	FVector3f BoundsSize;
	FVector3f ElevationColorBottom;
	FVector3f ElevationColorTop;
	int32 bUseCircle;
	int32 bUseColorOverride;
	int32 bUseElevationColor;
	FVector4f Offset;
	FVector4f Contrast;
	FVector4f Saturation;
	FVector4f Gamma;
	FVector3f Tint;
	float IntensityInfluence;
	int32 bUseClassification;
	int32 bUseClassificationAlpha;
	FVector4f ClassificationColors[32];
	FMatrix44f ClippingVolume[16];
	uint32 NumClippingVolumes;
	uint32 bStartClipped;

	FLidarPointCloudBatchElementUserData();

	void SetClassificationColors(const TMap<int32, FLinearColor>& InClassificationColors);
};

/**
 * Binds shader parameters necessary for rendering
 */
class FLidarPointCloudVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FLidarPointCloudVertexFactoryShaderParameters, NonVirtual);

public:
	void Bind(const FShaderParameterMap& ParameterMap);
	void GetElementShaderBindings(const class FSceneInterface* Scene, const FSceneView* View, const FMeshMaterialShader* Shader, const EVertexInputStreamType InputStreamType, ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory, const FMeshBatchElement& BatchElement, class FMeshDrawSingleShaderBindings& ShaderBindings, FVertexInputStreamArray& VertexStreams) const;

	LAYOUT_FIELD(FShaderResourceParameter, TreeBuffer);
	LAYOUT_FIELD(FShaderParameter, bEditorView);
	LAYOUT_FIELD(FShaderParameter, SelectionColor);
	LAYOUT_FIELD(FShaderParameter, LocationOffset);
	LAYOUT_FIELD(FShaderParameter, RootCellSize);
	LAYOUT_FIELD(FShaderParameter, RootExtent);
	LAYOUT_FIELD(FShaderParameter, bUsePerPointScaling);
	LAYOUT_FIELD(FShaderParameter, VirtualDepth);
	LAYOUT_FIELD(FShaderParameter, SpriteSizeMultiplier);
	LAYOUT_FIELD(FShaderParameter, ReversedVirtualDepthMultiplier);
	LAYOUT_FIELD(FShaderParameter, ViewRightVector);
	LAYOUT_FIELD(FShaderParameter, ViewUpVector);
	LAYOUT_FIELD(FShaderParameter, bUseCameraFacing);
	LAYOUT_FIELD(FShaderParameter, bUseScreenSizeScaling);
	LAYOUT_FIELD(FShaderParameter, bUseStaticBuffers);
	LAYOUT_FIELD(FShaderParameter, BoundsSize);
	LAYOUT_FIELD(FShaderParameter, ElevationColorBottom);
	LAYOUT_FIELD(FShaderParameter, ElevationColorTop);
	LAYOUT_FIELD(FShaderParameter, bUseCircle);
	LAYOUT_FIELD(FShaderParameter, bUseColorOverride);
	LAYOUT_FIELD(FShaderParameter, bUseElevationColor);
	LAYOUT_FIELD(FShaderParameter, Offset);
	LAYOUT_FIELD(FShaderParameter, Contrast);
	LAYOUT_FIELD(FShaderParameter, Saturation);
	LAYOUT_FIELD(FShaderParameter, Gamma);
	LAYOUT_FIELD(FShaderParameter, Tint);
	LAYOUT_FIELD(FShaderParameter, IntensityInfluence);
	LAYOUT_FIELD(FShaderParameter, bUseClassification);
	LAYOUT_FIELD(FShaderParameter, bUseClassificationAlpha);
	LAYOUT_FIELD(FShaderParameter, ClassificationColors);
	LAYOUT_FIELD(FShaderParameter, ClippingVolume);
	LAYOUT_FIELD(FShaderParameter, NumClippingVolumes);
	LAYOUT_FIELD(FShaderParameter, bStartClipped);
};

class FLidarPointCloudVertexFactoryBase : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FLidarPointCloudVertexFactoryBase);
	
public:
	static bool ShouldCache(const FVertexFactoryShaderPermutationParameters& Parameters) { return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5); }
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	FLidarPointCloudVertexFactoryBase() : FVertexFactory(ERHIFeatureLevel::SM5) { }

	virtual FVertexBuffer* GetVertexBuffer() { return nullptr; }
	virtual FBufferRHIRef GetVertexBufferRHI() { return GetVertexBuffer() ? GetVertexBuffer()->VertexBufferRHI : nullptr; }
	virtual FShaderResourceViewRHIRef GetVertexBufferSRV();
	virtual FRHIUniformBuffer* GetUniformBuffer() const { return UniformBuffer.GetReference(); }
	
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

protected:
	TUniformBufferRef<FLidarPointCloudVertexFactoryUniformShaderParameters> UniformBuffer;
};

class FLidarPointCloudVertexFactory : public FLidarPointCloudVertexFactoryBase
{
public:
	void Initialize(FLidarPointCloudPoint* Data, int32 NumPoints);
	
	virtual FVertexBuffer* GetVertexBuffer() override { return &VertexBuffer; }
	virtual FShaderResourceViewRHIRef GetVertexBufferSRV() override { return VertexBuffer.VertexBufferSRV; }

private:
	class FPointCloudVertexBuffer : public FVertexBuffer
	{
		FLidarPointCloudPoint* Data;
		int32 NumPoints;
		FShaderResourceViewRHIRef VertexBufferSRV;

	public:
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
		virtual FString GetFriendlyName() const override { return TEXT("FPointCloudVertexBuffer"); }
		friend FLidarPointCloudVertexFactory;
	} VertexBuffer;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
};

/**
 * Implementation of the custom Vertex Factory, containing only a ZeroStride position stream.
 */
class FLidarPointCloudSharedVertexFactory : public FLidarPointCloudVertexFactoryBase
{
public:
	virtual FVertexBuffer* GetVertexBuffer() override { return &VertexBuffer; }

private:
	/** Very simple implementation of a ZeroStride Vertex Buffer */
	class FPointCloudVertexBuffer : public FVertexBuffer
	{
		FShaderResourceViewRHIRef VertexBufferSRV;
	public:
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
		virtual FString GetFriendlyName() const override { return TEXT("FPointCloudVertexBuffer"); }
	} VertexBuffer;
	
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
};

/** A set of global render resources shared between all Lidar Point Cloud proxies */
extern TGlobalResource<FLidarPointCloudIndexBuffer> GLidarPointCloudIndexBuffer;
extern TGlobalResource<FLidarPointCloudSharedVertexFactory> GLidarPointCloudSharedVertexFactory;
extern TGlobalResource<FLidarPointCloudRenderBuffer> GDummyLidarPointCloudRenderBuffer;
