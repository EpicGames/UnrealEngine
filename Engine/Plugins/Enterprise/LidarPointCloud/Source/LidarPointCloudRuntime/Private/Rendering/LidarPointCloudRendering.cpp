// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/LidarPointCloudRendering.h"
#include "LidarPointCloudRenderBuffers.h"
#include "LidarPointCloudComponent.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudOctree.h"
#include "LidarPointCloudLODManager.h"
#include "PrimitiveSceneProxy.h"
#include "MeshBatch.h"
#include "Engine/Engine.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "LocalVertexFactory.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneView.h"

#if RHI_RAYTRACING
#include "RayTracingInstance.h"
#endif

DECLARE_DWORD_COUNTER_STAT(TEXT("Draw Calls"), STAT_DrawCallCount, STATGROUP_LidarPointCloud)

bool FLidarPointCloudProxyUpdateDataNode::BuildDataCache(bool bUseStaticBuffers, bool bUseRayTracing)
{
	if(DataNode && DataNode->BuildDataCache(bUseStaticBuffers, bUseRayTracing))
	{
		VertexFactory = DataNode->GetVertexFactory();
		DataCache = DataNode->GetDataCache();
		RayTracingGeometry = DataNode->GetRayTracingGeometry();
		DataNode = nullptr;
		return  true;
	}
	
	return false;
}

FLidarPointCloudProxyUpdateData::FLidarPointCloudProxyUpdateData()
	: NumElements(0)
	, VDMultiplier(1)
	, RootCellSize(1)
{
}

class FLidarPointCloudCollisionRendering
{
public:
	FLidarPointCloudCollisionRendering()
		: NumPrimitives(0)
		, MaxVertexIndex(0)
	{
	}
	~FLidarPointCloudCollisionRendering()
	{
		Release();
	}

	void Initialize(FLidarPointCloudOctree* Octree)
	{
		// Create collision visualization buffers
		if (Octree->HasCollisionData())
		{
			const FTriMeshCollisionData* CollisionData = Octree->GetCollisionData();
			
			// Initialize the buffers
			VertexBuffer.Initialize(CollisionData->Vertices.GetData(), CollisionData->Vertices.Num());
			VertexFactory.Initialize(&VertexBuffer);
			IndexBuffer.Initialize((int32*)CollisionData->Indices.GetData(), CollisionData->Indices.Num() * 3);

			NumPrimitives = CollisionData->Indices.Num();
			MaxVertexIndex = CollisionData->Vertices.Num() - 1;
		}
	}
	void Release()
	{
		VertexFactory.ReleaseResource();
		VertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
	}

	const FVertexFactory* GetVertexFactory() const { return &VertexFactory; }
	const FIndexBuffer* GetIndexBuffer() const { return &IndexBuffer; }
	const int32 GetNumPrimitives() const { return NumPrimitives; }
	const int32 GetMaxVertexIndex() const { return MaxVertexIndex; }

	bool ShouldRenderCollision() const
	{
		return NumPrimitives > 0 && VertexFactory.IsInitialized();
	}

private:
	class FLidarPointCloudCollisionVertexFactory : public FLocalVertexFactory
	{
	public:
		FLidarPointCloudCollisionVertexFactory() : FLocalVertexFactory(ERHIFeatureLevel::SM5, "") { }

		void Initialize(FVertexBuffer* InVertexBuffer)
		{
			FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

			FDataType NewData;
			NewData.PositionComponent = FVertexStreamComponent(InVertexBuffer, 0, 12, VET_Float3);
			NewData.ColorComponent = FVertexStreamComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
			NewData.TangentBasisComponents[0] = FVertexStreamComponent(&GNullColorVertexBuffer, 0, 0, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
			NewData.TangentBasisComponents[1] = FVertexStreamComponent(&GNullColorVertexBuffer, 0, 0, VET_PackedNormal, EVertexStreamUsage::ManualFetch);

			if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
			{
				NewData.PositionComponentSRV = RHICmdList.CreateShaderResourceView(
					InVertexBuffer->VertexBufferRHI, 
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_R32_FLOAT));
				NewData.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
				NewData.TangentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
				NewData.TextureCoordinatesSRV = GNullColorVertexBuffer.VertexBufferSRV;
			}

			Data = NewData;
			InitResource(RHICmdList);
		}
	} VertexFactory;
	class FLidarPointCloudCollisionVertexBuffer : public FVertexBuffer
	{
		const FVector3f* Data;
		int32 DataLength;

	public:
		void Initialize(const FVector3f* InData, int32 InDataLength)
		{
			Data = InData;
			DataLength = InDataLength;

			InitResource(FRHICommandListImmediate::Get());
		}

		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			const uint32 Size = DataLength * sizeof(FVector3f);

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("FLidarPointCloudCollisionVertexBuffer"), Size)
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
				.SetInitActionInitializer();

			FRHIBufferInitializer InitialData = RHICmdList.CreateBufferInitializer(CreateDesc);
			InitialData.WriteData(Data, Size);

			VertexBufferRHI = InitialData.Finalize();
		}
	} VertexBuffer;
	class FLidarPointCloudCollisionIndexBuffer : public FIndexBuffer
	{
		const int32* Data;
		int32 DataLength;

	public:
		void Initialize(const int32* InData, int32 InDataLength)
		{
			Data = InData;
			DataLength = InDataLength;

			InitResource(FRHICommandListImmediate::Get());
		}

		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			const uint32 Size = DataLength * sizeof(uint32);

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateIndex(TEXT("FLidarPointCloudCollisionIndexBuffer"), Size, sizeof(uint32))
				.AddUsage(EBufferUsageFlags::Static)
				.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
				.SetInitActionInitializer();

			FRHIBufferInitializer InitialData = RHICmdList.CreateBufferInitializer(CreateDesc);
			InitialData.WriteData(Data, Size);

			IndexBufferRHI = InitialData.Finalize();
		}
	} IndexBuffer;

	int32 NumPrimitives;
	int32 MaxVertexIndex;
};

class FLidarOneFrameResource : public FOneFrameResource
{
public:
	TArray<FLidarPointCloudBatchElementUserData> Payload;
	virtual ~FLidarOneFrameResource() {}
};

class FLidarPointCloudSceneProxy : public ILidarPointCloudSceneProxy, public FPrimitiveSceneProxy
{
public:
	FLidarPointCloudSceneProxy(ULidarPointCloudComponent* Component)
		: FPrimitiveSceneProxy(Component)
		, ProxyWrapper(MakeShared<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe>(this))
		, bCompatiblePlatform(GetScene().GetFeatureLevel() >= ERHIFeatureLevel::SM5)
		, Owner(Component->GetOwner())
		, CollisionRenderingPtr(&Component->GetPointCloud()->CollisionRendering)
	{
		// Skip material verification - async update could occasionally cause it to crash
		bVerifyUsedMaterials = false;

		TreeBuffer = new FLidarPointCloudRenderBuffer();

		MaterialRelevance = Component->GetMaterialRelevance(GetScene().GetShaderPlatform());
	}
	virtual ~FLidarPointCloudSceneProxy()
	{
		// Proxy is accessed only via RT, so there should not be any concurrency issues here
		ProxyWrapper->Proxy = nullptr;
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PointCloudSceneProxy_GetDynamicMeshElements);

		if (!CanBeRendered() || !RenderData.RenderParams.Material)
		{
			return;
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FSceneView* View = Views[ViewIndex];

			if (IsShown(View) && (VisibilityMap & (1 << ViewIndex)))
			{
				// Prepare the draw call
				if (RenderData.NumElements)
				{
					TArray<FLidarPointCloudBatchElementUserData>& UserData = Collector.AllocateOneFrameResource<FLidarOneFrameResource>().Payload;
					UserData.Reserve(RenderData.SelectedNodes.Num());

					for (const FLidarPointCloudProxyUpdateDataNode& Node : RenderData.SelectedNodes)
					{
						if ((RenderData.bUseStaticBuffers && Node.VertexFactory.IsValid() && Node.VertexFactory->IsInitialized()) ||
							(!RenderData.bUseStaticBuffers && Node.DataCache.IsValid()))
						{
							FMeshBatch& MeshBatch = Collector.AllocateMesh();
							
							SetupMeshBatch(MeshBatch, Node, &UserData[UserData.Add(BuildUserDataElement(View, Node))]);

							Collector.AddMesh(ViewIndex, MeshBatch);

							INC_DWORD_STAT(STAT_DrawCallCount);
						}
					}
				}

#if !(UE_BUILD_SHIPPING)
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

				// Draw selected nodes' bounds
				if (RenderData.RenderParams.bDrawNodeBounds)
				{
					for (const FBox& Node : RenderData.Bounds)
					{
						DrawWireBox(PDI, Node, FColor(72, 72, 255), SDPG_World);
					}
				}

				// Render bounds
				if (ViewFamily.EngineShowFlags.Bounds)
				{
					RenderBounds(PDI, ViewFamily.EngineShowFlags, GetBounds(), !Owner || IsSelected());
				}

				// Render collision wireframe
				FLidarPointCloudCollisionRendering* CollisionRendering = *CollisionRenderingPtr;
				if (ViewFamily.EngineShowFlags.Collision && IsCollisionEnabled() && CollisionRendering && CollisionRendering->ShouldRenderCollision())
				{
					// Create colored proxy
					FColoredMaterialRenderProxy* CollisionMaterialInstance;
					CollisionMaterialInstance = new FColoredMaterialRenderProxy(GEngine->WireframeMaterial->GetRenderProxy(), FColor(0, 255, 255, 255));
					Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);

					FMeshBatch& MeshBatch = Collector.AllocateMesh();
					MeshBatch.Type = PT_TriangleList;
					MeshBatch.VertexFactory = CollisionRendering->GetVertexFactory();
					MeshBatch.bWireframe = true;
					MeshBatch.MaterialRenderProxy = CollisionMaterialInstance;
					MeshBatch.ReverseCulling = !IsLocalToWorldDeterminantNegative();
					MeshBatch.DepthPriorityGroup = SDPG_World;
					MeshBatch.CastShadow = false;

					FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
					BatchElement.IndexBuffer = CollisionRendering->GetIndexBuffer();
					BatchElement.FirstIndex = 0;
					BatchElement.NumPrimitives = CollisionRendering->GetNumPrimitives();
					BatchElement.MinVertexIndex = 0;
					BatchElement.MaxVertexIndex = CollisionRendering->GetMaxVertexIndex();

					Collector.AddMesh(ViewIndex, MeshBatch);
				}
#endif // !(UE_BUILD_SHIPPING)
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;

		if (CanBeRendered())
		{
			Result.bDrawRelevance = IsShown(View);
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bDynamicRelevance = true;
			Result.bStaticRelevance = false;
			Result.bRenderInMainPass = ShouldRenderInMainPass();
			Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
			Result.bRenderCustomDepth = ShouldRenderCustomDepth();
			MaterialRelevance.SetPrimitiveViewRelevance(Result);
			Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		}

		return Result;
	}

	/** UserData is used to pass rendering information to the VertexFactory */
	FLidarPointCloudBatchElementUserData BuildUserDataElement(const FSceneView* InView, const FLidarPointCloudProxyUpdateDataNode& Node) const
	{	
		FLidarPointCloudBatchElementUserData UserDataElement;

		const bool bUsesSprites = RenderData.RenderParams.PointSize > 0;

		// Update shader parameters
		UserDataElement.bEditorView = RenderData.RenderParams.bOwnedByEditor;
		UserDataElement.ReversedVirtualDepthMultiplier = RenderData.VDMultiplier;
		UserDataElement.VirtualDepth = RenderData.VDMultiplier * Node.VirtualDepth;
		UserDataElement.SpriteSizeMultiplier = bUsesSprites ? (RenderData.RenderParams.PointSize + RenderData.RenderParams.GapFillingStrength * 0.005f) * RenderData.RenderParams.ComponentScale : 0;
		UserDataElement.bUseScreenSizeScaling = RenderData.RenderParams.ScalingMethod == ELidarPointCloudScalingMethod::FixedScreenSize;;
		UserDataElement.bUsePerPointScaling = RenderData.RenderParams.ScalingMethod == ELidarPointCloudScalingMethod::PerPoint;
		UserDataElement.bUseStaticBuffers = RenderData.bUseStaticBuffers;
		UserDataElement.RootCellSize = RenderData.RootCellSize;
		UserDataElement.RootExtent = FVector3f(RenderData.RenderParams.BoundsSize.GetAbsMax() * 0.5f);

		UserDataElement.LocationOffset = RenderData.RenderParams.LocationOffset;
		UserDataElement.ViewRightVector = InView ? (FVector3f)InView->GetViewRight() : FVector3f::RightVector;
		UserDataElement.ViewUpVector = InView ? (FVector3f)InView->GetViewUp() : FVector3f::ForwardVector;
		UserDataElement.bUseCameraFacing = !RenderData.RenderParams.bShouldRenderFacingNormals;
		UserDataElement.BoundsSize = RenderData.RenderParams.BoundsSize;
		UserDataElement.ElevationColorBottom = FVector3f(RenderData.RenderParams.ColorSource == ELidarPointCloudColorationMode::None ? FColor::White : RenderData.RenderParams.ElevationColorBottom);
		UserDataElement.ElevationColorTop = FVector3f(RenderData.RenderParams.ColorSource == ELidarPointCloudColorationMode::None ? FColor::White : RenderData.RenderParams.ElevationColorTop);
		UserDataElement.bUseCircle = bUsesSprites && RenderData.RenderParams.PointShape == ELidarPointCloudSpriteShape::Circle;
		UserDataElement.bUseColorOverride = RenderData.RenderParams.ColorSource != ELidarPointCloudColorationMode::Data && RenderData.RenderParams.ColorSource != ELidarPointCloudColorationMode::DataWithClassificationAlpha;
		UserDataElement.bUseElevationColor = RenderData.RenderParams.ColorSource == ELidarPointCloudColorationMode::Elevation || RenderData.RenderParams.ColorSource == ELidarPointCloudColorationMode::None;
		UserDataElement.Offset = RenderData.RenderParams.Offset;
		UserDataElement.Contrast = RenderData.RenderParams.Contrast;
		UserDataElement.Saturation = RenderData.RenderParams.Saturation;
		UserDataElement.Gamma = RenderData.RenderParams.Gamma;
		UserDataElement.Tint = RenderData.RenderParams.ColorTint;
		UserDataElement.IntensityInfluence = RenderData.RenderParams.IntensityInfluence;

		UserDataElement.bUseClassification = RenderData.RenderParams.ColorSource == ELidarPointCloudColorationMode::Classification;
		UserDataElement.bUseClassificationAlpha = RenderData.RenderParams.ColorSource == ELidarPointCloudColorationMode::DataWithClassificationAlpha;
		UserDataElement.SetClassificationColors(RenderData.RenderParams.ClassificationColors);
				
		UserDataElement.NumClippingVolumes = FMath::Min(RenderData.ClippingVolumes.Num(), 16);

		for (uint32 i = 0; i < UserDataElement.NumClippingVolumes; ++i)
		{
			const FLidarPointCloudClippingVolumeParams& ClippingVolume = RenderData.ClippingVolumes[i];

			UserDataElement.ClippingVolume[i] = FMatrix44f(ClippingVolume.PackedShaderData);
			UserDataElement.bStartClipped |= ClippingVolume.Mode == ELidarClippingVolumeMode::ClipOutside;
		}

		UserDataElement.TreeBuffer = TreeBuffer->SRV;

		return UserDataElement;
	}

	virtual bool CanBeOccluded() const override { return !MaterialRelevance.bDisableDepthTest; }
	virtual uint32 GetMemoryFootprint() const override { return(sizeof(*this) + GetAllocatedSize()); }
	uint32 GetAllocatedSize() const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }
	bool CanBeRendered() const { return bCompatiblePlatform; }

	virtual SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	virtual void UpdateRenderData(const FLidarPointCloudProxyUpdateData& InRenderData) override
	{
		RenderData = InRenderData;
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

		const int32 NumTreeStructure = RenderData.TreeStructure.Num() > 0 ? RenderData.TreeStructure.Num() : 16;
		TreeBuffer->Resize(NumTreeStructure);
		uint8* DataPtr = (uint8*)RHICmdList.LockBuffer(TreeBuffer->Buffer, 0, NumTreeStructure * sizeof(uint32), RLM_WriteOnly);
		FMemory::Memzero(DataPtr, NumTreeStructure * sizeof(uint32));
		if (RenderData.TreeStructure.Num() > 0)
		{
			FMemory::Memcpy(DataPtr, RenderData.TreeStructure.GetData(), NumTreeStructure * sizeof(uint32));
		}
		RHICmdList.UnlockBuffer(TreeBuffer->Buffer);
	}

	void SetupMeshBatch(FMeshBatch& MeshBatch, const FLidarPointCloudProxyUpdateDataNode& Node, const FLidarPointCloudBatchElementUserData* UserData) const
	{
		const bool bUsesSprites = RenderData.RenderParams.PointSize > 0;
		
		MeshBatch.Type = bUsesSprites ? PT_TriangleList : PT_PointList;
		MeshBatch.LODIndex = 0;
		MeshBatch.VertexFactory = RenderData.bUseStaticBuffers ? Node.VertexFactory.Get() : (FVertexFactory*)&GLidarPointCloudSharedVertexFactory;
		MeshBatch.bWireframe = false;
		MeshBatch.MaterialRenderProxy = RenderData.RenderParams.Material->GetRenderProxy();
		MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
		MeshBatch.DepthPriorityGroup = SDPG_World;
		MeshBatch.bCanApplyViewModeOverrides = true;
		
		FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
		BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
		BatchElement.IndexBuffer = &GLidarPointCloudIndexBuffer;
		BatchElement.FirstIndex = bUsesSprites ? 0 : GLidarPointCloudIndexBuffer.PointOffset;
		BatchElement.MinVertexIndex = 0;
		BatchElement.NumPrimitives = Node.NumVisiblePoints * (bUsesSprites ? 2 : 1);
		BatchElement.MaxVertexIndex = Node.NumVisiblePoints * (bUsesSprites ? 4 : 1);
		BatchElement.UserData = UserData;
		BatchElement.VertexFactoryUserData = RenderData.bUseStaticBuffers ? Node.VertexFactory->GetUniformBuffer() : Node.DataCache->GetUniformBuffer();
	}

#if RHI_RAYTRACING
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool HasRayTracingRepresentation() const override { return true; }

	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override
	{
		if (RenderData.NumElements)
		{
			TConstArrayView<const FSceneView*> Views = Collector.GetViews();
			const uint32 VisibilityMap = Collector.GetVisibilityMap();

			// RT geometry will be generated based on first active view and then reused for all other views
			// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
			const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
			checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

			const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

			TArray<FLidarPointCloudBatchElementUserData>& UserData = Collector.AllocateOneFrameResource<FLidarOneFrameResource>().Payload;
			UserData.Reserve(RenderData.SelectedNodes.Num());

			CachedRayTracingMaterials.Reset();
			const FMatrix& ThisLocalToWorld = GetLocalToWorld();

			for (const FLidarPointCloudProxyUpdateDataNode& Node : RenderData.SelectedNodes)
			{
				if (Node.RayTracingGeometry.IsValid() && Node.RayTracingGeometry->IsInitialized())
				{
					TArray<FMeshBatch> &NodeRayTracingMaterials = CachedRayTracingMaterials.AddDefaulted_GetRef();
					FMeshBatch &MeshBatch = NodeRayTracingMaterials.AddDefaulted_GetRef();
					SetupMeshBatch(MeshBatch, Node, &UserData[UserData.Add(BuildUserDataElement(nullptr, Node))]);
					MeshBatch.SegmentIndex = 0;
					MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);

					FLidarPointCloudRayTracingGeometry* Geometry = Node.RayTracingGeometry.Get();

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						if ((VisibilityMap & (1 << ViewIndex)) == 0)
						{
							continue;
						}

						FRayTracingInstance RayTracingInstance;
						RayTracingInstance.Geometry = Geometry;
						RayTracingInstance.InstanceTransformsView = MakeArrayView(&ThisLocalToWorld, 1);
						RayTracingInstance.MaterialsView = MakeArrayView(NodeRayTracingMaterials);

						Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
					}

					{
						FRayTracingDynamicGeometryUpdateParams UpdateParams;
						UpdateParams.MeshBatchesView = NodeRayTracingMaterials;
						UpdateParams.NumVertices = Geometry->GetNumVertices();
						UpdateParams.VertexBufferSize = Geometry->GetBufferSize();
						UpdateParams.NumTriangles = Geometry->GetNumPrimitives();
						UpdateParams.Geometry = Geometry;

						Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
					}
				}
			}
		}
	}
#endif // RHI_RAYTRACING
	
public:
	TSharedPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> ProxyWrapper;

private:
	FLidarPointCloudProxyUpdateData RenderData;

	FLidarPointCloudRenderBuffer* TreeBuffer;
	FMaterialRelevance MaterialRelevance;

	TArray<TArray<FMeshBatch>> CachedRayTracingMaterials;
	
	bool bCompatiblePlatform;
	
	AActor* Owner;

	FLidarPointCloudCollisionRendering** CollisionRenderingPtr;
};

FPrimitiveSceneProxy* ULidarPointCloudComponent::CreateSceneProxy()
{
	FLidarPointCloudSceneProxy* Proxy = nullptr;
	if (PointCloud)
	{
		Proxy = new FLidarPointCloudSceneProxy(this);
		
		if (Proxy->CanBeRendered())
		{
			FLidarPointCloudLODManager::RegisterProxy(this, Proxy->ProxyWrapper);
		}
	}
	return Proxy;
}

void ULidarPointCloud::InitializeCollisionRendering()
{
	// Do not process, if the app is incapable of rendering
	if (!FApp::CanEverRender())
	{
		return;
	}

	if (IsInRenderingThread())
	{
		FScopeLock Lock(&Octree.DataLock);

		if (!CollisionRendering)
		{
			CollisionRendering = new FLidarPointCloudCollisionRendering();
		}

		CollisionRendering->Initialize(&Octree);
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(InitializeCollisionRendering)(
			[this](FRHICommandListImmediate& RHICmdList)
			{
				InitializeCollisionRendering();
			});
	}
}

void ULidarPointCloud::ReleaseCollisionRendering(bool bDestroyAfterRelease)
{
	// Do not process, if the app in incapable of rendering
	if (!FApp::CanEverRender())
	{
		return;
	}

	if (IsInRenderingThread())
	{
		if (CollisionRendering)
		{
			if (bDestroyAfterRelease)
			{
				delete CollisionRendering;
				CollisionRendering = nullptr;
			}
			else
			{
				CollisionRendering->Release();
			}
		}
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(ReleaseCollisionRendering)(
			[this, bDestroyAfterRelease](FRHICommandListImmediate& RHICmdList)
			{
				ReleaseCollisionRendering(bDestroyAfterRelease);
			});
	}
}
