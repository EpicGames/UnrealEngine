// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"
#include "Widgets/SWidget.h"
#include "Exporters/Exporter.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/Paths.h"
#include "LidarPointCloudFileIO.generated.h"

/** Base for all importer settings */
struct LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudImportSettings
{
public:
	/** Holds a flag determining whether the same settings should be applied to the whole import at once */
	bool bImportAll;

protected:
	/** Used to determine the correct Handler to use during serialization. */
	FString Filename;

public:
	FLidarPointCloudImportSettings(const FString& Filename)
		: bImportAll(false)
		, Filename(Filename)
	{
	}
	virtual ~FLidarPointCloudImportSettings() {}

	/**
	 * Should return true if the given file is compatible with this instance of settings
	 * Useful for detecting different headers
	 */
	virtual bool IsFileCompatible(const FString& InFilename) const { return false; }

	/**
	 * Links the FLidarPointCloudImportSettings with FArchive serialization
	 * No need to manually serialize Filename - it is handled by ULidarPointCloudFileIO::SerializeImportSettings
	 */
	virtual void Serialize(FArchive& Ar);

	FString GetFilename() const { return Filename; }

	/** Returns true it this is an instance of FLidarPointCloudImportSettings */
	bool IsGeneric() const { return GetUID().Equals(FLidarPointCloudImportSettings("").GetUID()); }

	virtual void SetNewFilename(const FString& NewFilename)
	{
		Filename = NewFilename;
	}

	/** Must return a unique id of this Import Settings type. */
	virtual FString GetUID() const { return "FLidarPointCloudImportSettings"; }

	/**
	 * Returns duplicate of this instance of Import Settings.
	 * Optionally, can use a different filename when cloning.
	 */
	virtual TSharedPtr<FLidarPointCloudImportSettings> Clone(const FString& NewFilename = "") { return nullptr; }

	static TSharedPtr<FLidarPointCloudImportSettings> MakeGeneric(const FString& Filename)
	{
		return TSharedPtr<FLidarPointCloudImportSettings>(new FLidarPointCloudImportSettings(Filename));
	}

#if WITH_EDITOR
	/** Used to create properties window. */
	virtual TSharedPtr<SWidget> GetWidget() { return nullptr; }

	/** Should return true, if the importer requires import settings UI to be shown. */
	virtual bool HasImportUI() const { return false; }
#endif

	friend class ULidarPointCloudFileIO;
	friend class FLidarPointCloudFileIOHandler;
};

/** Stores the results of the import process */
struct LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudImportResults
{
public:
	TArray64<FLidarPointCloudPoint> Points;
	FBox Bounds;
	FVector OriginalCoordinates;

	/** Contains the list of imported classification IDs */
	TArray<uint8> ClassificationsImported;

private:
	/** Used for async importing */
	TFunction<void(float)> ProgressCallback;
	FThreadSafeBool* bCancelled;
	uint64 ProgressFrequency;
	uint64 ProgressCounter;
	uint64 TotalProgressCounter;
	uint64 MaxProgressCounter;
	TFunction<void(const FBox& Bounds, FVector)> InitCallback;
	TFunction<void(TArray64<FLidarPointCloudPoint>*)> BufferCallback;

public:
	FLidarPointCloudImportResults(FThreadSafeBool* bInCancelled = nullptr, TFunction<void(float)> InProgressCallback = TFunction<void(float)>());
	FLidarPointCloudImportResults(FThreadSafeBool* bInCancelled, TFunction<void(float)> InProgressCallback, TFunction<void(const FBox& Bounds, FVector)> InInitCallback, TFunction<void(TArray64<FLidarPointCloudPoint>*)> InBufferCallback);

	void InitializeOctree(const FBox& InBounds);

	void ProcessBuffer(TArray64<FLidarPointCloudPoint>* InPoints);

	void SetPointCount(const uint64& InTotalPointCount);

	inline void AddPoint(const FVector& Location, const float& R, const float& G, const float& B, const float& A = 1.0f)
	{
		AddPoint(Location.X, Location.Y, Location.Z, R, G, B, A);
	}

	void AddPoint(const float& X, const float& Y, const float& Z, const float& R, const float& G, const float& B, const float& A = 1.0f);

	void AddPoint(const float& X, const float& Y, const float& Z, const float& R, const float& G, const float& B, const float& A, const float& NX, const float& NY, const float& NZ);

	void AddPointsBulk(TArray64<FLidarPointCloudPoint>& InPoints);

	void CenterPoints();

	inline bool IsCancelled() const
	{
		return bCancelled && *bCancelled;
	}

	void SetMaxProgressCounter(uint64 MaxCounter);

	void IncrementProgressCounter(uint64 Increment);
};

/** Base type implemented by all file handlers. */
class LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudFileIOHandler
{
protected:
	/** Used for precision loss check and correction. */
	double PrecisionCorrectionOffset[3] = { 0, 0, 0 };
	bool bPrecisionCorrected = false;

public:
	/** Called before importing to prepare the data. */
	void PrepareImport();

	/** Must return true if the handler supports importing. */
	virtual bool SupportsImport() const { return false; }

	/** Must return true if the handler supports exporting. */
	virtual bool SupportsExport() const { return false; }

	/** This is what will actually be called to process the import of the file. */
	virtual bool HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults) { return false; }

	/** These will actually be called to process the export of the asset. */
	virtual bool HandleExport(const FString& Filename, class ULidarPointCloud* PointCloud) { return false; }

	/** Returns a shared pointer for the import settings of this importer. */
	virtual TSharedPtr<FLidarPointCloudImportSettings> GetImportSettings(const FString& Filename) const { return TSharedPtr<FLidarPointCloudImportSettings>(new FLidarPointCloudImportSettings(Filename)); }

	/**
	 * Returns true if the given file supports importing and building an octree concurrently.
	 * False if the data needs caching first.
	 */
	virtual bool SupportsConcurrentInsertion(const FString& Filename) const { return false; }

	/**
	 * Must return true if the provided UID is of supported Import Settings type.
	 * Default implementation simply checks the default ImportSettings' UID to compare against.
	 */
	virtual bool IsSettingsUIDSupported(const FString& UID) { return UID.Equals(GetImportSettings("")->GetUID()); }

	/**
	 * Performs validation checks and corrections on the provided ImportSettings object using the given Filename.
	 * Returns true if the resulting object is valid.
	 */
	virtual bool ValidateImportSettings(TSharedPtr<FLidarPointCloudImportSettings>& ImportSettings, const FString& Filename);
};

/**
 * Holds information about all registered file handlers
 */
UCLASS()
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudFileIO : public UExporter
{
	GENERATED_UCLASS_BODY()

private:
	/** Links extensions to file handlers. */
	static TMap<FString, FLidarPointCloudFileIOHandler*> RegisteredHandlers;

public:
	/**
	 * Automatically detects correct format and performs the import.
	 * Returns true if the import was successful.
	 */
	static bool Import(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults);

	static bool Export(const FString& Filename, class ULidarPointCloud* AssetToExport);

	/**
	 * Returns the import settings instance depending on the format provided
	 */
	static TSharedPtr<FLidarPointCloudImportSettings> GetImportSettings(const FString& Filename);

	/** Returns the list of all registered import extensions */
	static TArray<FString> GetSupportedImportExtensions();

	/** Returns the list of all registered export extensions */
	static TArray<FString> GetSupportedExportExtensions();

	/** Called to register an handler for the supported formats list */
	static void RegisterHandler(FLidarPointCloudFileIOHandler* Handler, const TArray<FString>& Extensions);

	/** Returns pointer to the handler, which supports the given format or null if none found. */
	static FLidarPointCloudFileIOHandler* FindHandlerByFilename(const FString& Filename) { return FindHandlerByType(FPaths::GetExtension(Filename)); }
	static FLidarPointCloudFileIOHandler* FindHandlerByType(const FString& Type);

	/** Responsible for serialization using correct serializer for the given format. */
	static void SerializeImportSettings(FArchive& Ar, TSharedPtr<FLidarPointCloudImportSettings>& ImportSettings);

	/**
	 * Returns true if the given file supports importing and building an octree concurrently.
	 * False if the data needs caching first.
	 */
	static bool FileSupportsConcurrentInsertion(const FString& Filename);

public:
	// Begin UExporter Interface
	virtual bool SupportsObject(UObject* Object) const override;
	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0) override;
	// End UExporter Interface
};