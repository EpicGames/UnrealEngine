// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloud.h"
#include "LidarPointCloudFileIO.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/LatentActionManager.h"

#if WITH_EDITOR
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSpinBox.h"
#endif

#include "LidarPointCloudFileIO_ASCII.generated.h"

/**
 * This set of classes is responsible for importing column-based text formats with extensions XYZ, TXT and PTS. 
 */

/** Used to help expose the import settings to Blueprints */
USTRUCT(BlueprintType)
struct FLidarPointCloudImportSettings_ASCII_Columns
{
	GENERATED_BODY()

public:
	/** Index of a column containing Location X data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 LocationX;

	/** Index of a column containing Location Y data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 LocationY;

	/** Index of a column containing Location Z data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 LocationZ;

	/** Index of a column containing Red channel. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 Red;

	/** Index of a column containing Green channel. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 Green;

	/** Index of a column containing Blue channel. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 Blue;

	/** Index of a column containing Intensity channel. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 Intensity;

	/** Index of a column containing Normal X data. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 NormalX;

	/** Index of a column containing Normal Y data. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 NormalY;

	/** Index of a column containing Normal Z data. Set to -1 if not available */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Import Settings")
	int32 NormalZ;

public:
	FLidarPointCloudImportSettings_ASCII_Columns()
	{
		LocationX = 0;
		LocationY = 1;
		LocationZ = 2;
		Red = 3;
		Green = 4;
		Blue = 5;
		Intensity = 6;
		NormalX = -1;
		NormalY = -1;
		NormalZ = -1;
	}
};

struct FLidarPointCloudImportSettings_ASCII : public FLidarPointCloudImportSettings
{
	int32 LinesToSkip;
	int64 EstimatedPointCount;
	FString Delimiter;
	TArray<FString> Columns;
	TArray<int32> SelectedColumns;
	FVector2D RGBRange;

public:
	FLidarPointCloudImportSettings_ASCII(const FString& Filename);
	virtual bool IsFileCompatible(const FString& InFilename) const override;

	/** Links the FLidarPointCloudImportSettings_ASCII with FArchive serialization */
	virtual void Serialize(FArchive& Ar) override;

	virtual FString GetUID() const override { return "FLidarPointCloudImportSettings_ASCII"; }

	virtual void SetNewFilename(const FString& NewFilename) override
	{
		FLidarPointCloudImportSettings::SetNewFilename(NewFilename);

		int32 OldLinesToSkip = LinesToSkip;
		int32 OldColumnCount = Columns.Num();
		auto OldSelectedColumns = SelectedColumns;

		ReadFileHeader(NewFilename);

		// Restore the column selection if compatible
		if ((LinesToSkip == OldLinesToSkip) && (Columns.Num() == OldColumnCount))
		{
			SelectedColumns = OldSelectedColumns;
		}
	}

	virtual TSharedPtr<FLidarPointCloudImportSettings> Clone(const FString& NewFilename = "") override
	{
		TSharedPtr<FLidarPointCloudImportSettings_ASCII> NewSettings(new FLidarPointCloudImportSettings_ASCII(NewFilename.IsEmpty() ? Filename : NewFilename));

		NewSettings->bImportAll = bImportAll;
		NewSettings->SelectedColumns = SelectedColumns;
		NewSettings->RGBRange = RGBRange;

		return NewSettings;
	}

	inline bool HasValidData() { return !Delimiter.IsEmpty() && Columns.Num() > 0; }

private:
	/** Reads and parses header information about the given file. */
	void ReadFileHeader(const FString& InFilename);

	/**
	 * Scans the given file and searches for Min and Max values within the given columns.
	 * Optionally, attempts to find the best color range match for the given MinMax pair
	 */
	FVector2D ReadFileMinMaxColumns(TArray<int32> ColumnsToScan, bool bBestMatch);

	friend class ULidarPointCloudFileIO_ASCII;

#if WITH_EDITOR
private:
	TArray<TSharedPtr<FString>> Options;
	TSharedPtr<SSpinBox<float>> RGBRangeMin;
	TSharedPtr<SSpinBox<float>> RGBRangeMax;

public:
	virtual TSharedPtr<SWidget> GetWidget() override;

	virtual bool HasImportUI() const override { return true; }

private:
	TSharedRef<SWidget> HandleGenerateWidget(FString Item) const;
#endif
};

/**
 * Inherits from UBlueprintFunctionLibrary to allow exposure to Blueprint Library in the same class.
 */
UCLASS()
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudFileIO_ASCII : public UBlueprintFunctionLibrary, public FLidarPointCloudFileIOHandler
{
	GENERATED_BODY()

public:
	virtual bool SupportsImport() const override { return true; }
	virtual bool SupportsExport() const override { return true; }

	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo", ExpandEnumAsExecs = "AsyncMode", DisplayName = "Create Lidar Point Cloud From File (ASCII)"))
	static void CreatePointCloudFromFile(UObject* WorldContextObject, const FString& Filename, bool bUseAsync, FVector2D RGBRange, FLidarPointCloudImportSettings_ASCII_Columns Columns, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud);
	static ULidarPointCloud* CreatePointCloudFromFile(const FString& Filename, const FLidarPointCloudAsyncParameters& AsyncParameters, const FVector2D& RGBRange, const FLidarPointCloudImportSettings_ASCII_Columns& Columns);

	virtual bool HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults &OutImportResults) override;
	virtual TSharedPtr<FLidarPointCloudImportSettings> GetImportSettings(const FString& Filename) const override { return TSharedPtr<FLidarPointCloudImportSettings>(new FLidarPointCloudImportSettings_ASCII(Filename)); }

	virtual bool HandleExport(const FString& Filename, class ULidarPointCloud* PointCloud) override;

	virtual bool SupportsConcurrentInsertion(const FString& Filename) const override { return false; }

	ULidarPointCloudFileIO_ASCII() { ULidarPointCloudFileIO::RegisterHandler(this, { "TXT", "XYZ", "PTS" }); }
};