// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudFileIO.h"
#include "LidarPointCloudFileIO_E57.generated.h"

#if LIBE57SUPPORTED
struct FLidarPointCloudImportSettings_E57 : public FLidarPointCloudImportSettings
{
	FLidarPointCloudImportSettings_E57(const FString& Filename) : FLidarPointCloudImportSettings(Filename) { }
	virtual bool IsFileCompatible(const FString& InFilename) const override { return true; }
	virtual FString GetUID() const override { return "FLidarPointCloudImportSettings_E57"; }
	virtual TSharedPtr<FLidarPointCloudImportSettings> Clone(const FString& NewFilename = "") override
	{
		TSharedPtr<FLidarPointCloudImportSettings_E57> NewSettings(new FLidarPointCloudImportSettings_E57(NewFilename.IsEmpty() ? Filename : NewFilename));
		NewSettings->bImportAll = bImportAll;
		return NewSettings;
	}
};
#endif

UCLASS()
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudFileIO_E57 : public UObject, public FLidarPointCloudFileIOHandler
{
	GENERATED_BODY()

public:
#if LIBE57SUPPORTED
	virtual bool SupportsImport() const override { return true; }
	virtual bool SupportsExport() const override { return false; }

	virtual bool HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults) override;
	virtual bool HandleExport(const FString& Filename, class ULidarPointCloud* PointCloud) override { return false; }

	virtual TSharedPtr<FLidarPointCloudImportSettings> GetImportSettings(const FString& Filename) const override { return TSharedPtr<FLidarPointCloudImportSettings>(new FLidarPointCloudImportSettings_E57(Filename)); }

	virtual bool SupportsConcurrentInsertion(const FString& Filename) const override { return false; }
#endif

	ULidarPointCloudFileIO_E57()
	{
#if LIBE57SUPPORTED
		ULidarPointCloudFileIO::RegisterHandler(this, { "E57" });
#endif
	}
};
