// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudFileIO.h"
#include "LidarPointCloudFileIO_LAS.generated.h"

class ULidarPointCloud;

struct FLidarPointCloudImportSettings_LAS : public FLidarPointCloudImportSettings
{
	FLidarPointCloudImportSettings_LAS(const FString& Filename) : FLidarPointCloudImportSettings(Filename) { }
	virtual bool IsFileCompatible(const FString& InFilename) const override { return true; }
	virtual void Serialize(FArchive& Ar) override;
	virtual FString GetUID() const override { return "FLidarPointCloudImportSettings_LAS"; }
	virtual TSharedPtr<FLidarPointCloudImportSettings> Clone(const FString& NewFilename = "") override
	{
		TSharedPtr<FLidarPointCloudImportSettings_LAS> NewSettings(new FLidarPointCloudImportSettings_LAS(NewFilename.IsEmpty() ? Filename : NewFilename));
		NewSettings->bImportAll = bImportAll;
		return NewSettings;
	}
};

UCLASS()
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudFileIO_LAS : public UObject, public FLidarPointCloudFileIOHandler
{
	GENERATED_BODY()

public:
	virtual bool SupportsImport() const override { return true; }
	virtual bool SupportsExport() const override { return true; }
	
	virtual TSharedPtr<FLidarPointCloudImportSettings> GetImportSettings(const FString& Filename) const override { return TSharedPtr<FLidarPointCloudImportSettings>(new FLidarPointCloudImportSettings_LAS(Filename)); }

	virtual bool HandleImport(const FString& Filename, TSharedPtr<FLidarPointCloudImportSettings> ImportSettings, FLidarPointCloudImportResults& OutImportResults) override;
	virtual bool HandleExport(const FString& Filename, ULidarPointCloud* PointCloud) override;

	virtual bool SupportsConcurrentInsertion(const FString& Filename) const override;

	ULidarPointCloudFileIO_LAS()
	{
		ULidarPointCloudFileIO::RegisterHandler(this, { "LAS" });
#if LASZIPSUPPORTED
		ULidarPointCloudFileIO::RegisterHandler(this, { "LAZ" });
#endif
	}

private:
	bool HandleImportLAS(const FString& Filename, FLidarPointCloudImportResults& OutImportResults);
	bool HandleExportLAS(const FString& Filename, ULidarPointCloud* PointCloud);

#if LASZIPSUPPORTED
	bool HandleImportLAZ(const FString& Filename, FLidarPointCloudImportResults& OutImportResults);
	bool HandleExportLAZ(const FString& Filename, ULidarPointCloud* PointCloud);
#endif
};