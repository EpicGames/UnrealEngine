// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudOctree.h"
#include "HAL/ThreadSafeBool.h"
#include "Engine/EngineTypes.h"
#include "LidarPointCloudSettings.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/LatentActionManager.h"
#include "ConvexVolume.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "GameFramework/Volume.h"
#include "LidarPointCloud.generated.h"

class ALidarPointCloudActor;
class ULidarPointCloudComponent;
class UBodySetup;
class FLidarPointCloudCollisionRendering;
class FLidarPointCloudNotification;

namespace LidarPointCloudMeshing
{
	struct FMeshBuffers;
};

/**
 * Used for ULidarPointCloud::CreateFromXXXX calls
 */
struct FLidarPointCloudAsyncParameters
{
	/** Should the creation use async operation */
	bool bUseAsync;

	/**
	 * Called every time at least 1% progress is generated.
	 * The parameter is within 0.0 - 1.0 range.
	 */
	TFunction<void(float)> ProgressCallback;

	/**
	 * Called once, when the operation completes.
	 * The parameter specifies whether it has been executed successfully.
	 */
	TFunction<void(bool)> CompletionCallback;

	FLidarPointCloudAsyncParameters(bool bUseAsync, TFunction<void(float)> ProgressCallback = nullptr, TFunction<void(bool)> CompletionCallback = nullptr)
		: bUseAsync(bUseAsync)
		, ProgressCallback(MoveTemp(ProgressCallback))
		, CompletionCallback(MoveTemp(CompletionCallback))
	{
	}
};

/** Used to notify the component it should refresh its state. */
DECLARE_EVENT(ULidarPointCloud, FOnPointCloudChanged);

/**
 * Represents the Point Cloud asset
 */
UCLASS(BlueprintType, AutoExpandCategories=("Performance", "Rendering|Sprite"), AutoCollapseCategories=("Import Settings"))
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloud : public UObject, public IInterface_CollisionDataProvider
{
	GENERATED_BODY()

private:
	/** Stores the path to the original source file. Empty if dynamically created. */
	UPROPERTY(EditAnywhere, Category = "Import Settings", meta = (AllowPrivateAccess = "true"))
	FFilePath SourcePath;

public:
	/**
	 * Determines the maximum error (in cm) of the collision for this point cloud.
	 * NOTE: Lower values will require more time to build.
	 * Rebuild collision for the changes to take effect.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collision")
	float MaxCollisionError;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use MaxCollisionError instead."))
	float CollisionAccuracy_DEPRECATED;

	/** Higher values will generally result in more accurate calculations, at the expense of time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Normals", meta = (ClampMin = "1", ClampMax = "100", DisplayName = "Quality"))
	int32 NormalsQuality;

	/**
	 * Higher values are less susceptible to noise, but will most likely lose finer details, especially around hard edges.
	 * Lower values retain more detail, at the expense of time.
	 * NOTE: setting this too low will cause visual artifacts and geometry holes in noisier datasets.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Normals", meta = (ClampMin = "0.0", DisplayName = "Noise Tolerance"))
	float NormalsNoiseTolerance;

	/** Stores the original offset as a double. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Coordinates")
	FVector OriginalCoordinates;

private:
	/**
	 * Disables the LOD pipeline, allowing for much faster data operations (insert/remove/set)
	 * at a potential expense of runtime performance. The whole asset will be treated as a single,
	 * large asset with no granular density control, nor occlusion culling.
	 * 
	 * Recommended for assets, which have their data updated per-frame (such as live streaming).
	 */
	UPROPERTY(EditAnywhere, Category = "Performance", meta = (AllowPrivateAccess = "true"))
	bool bOptimizedForDynamicData;

public:
	/** Holds pointer to the Import Settings used for the import. */
	TSharedPtr<struct FLidarPointCloudImportSettings> ImportSettings;

	FLidarPointCloudOctree Octree;
	FLidarPointCloudCollisionRendering* CollisionRendering;

	/** Contains an offset to be added to all points when rendering */
	UPROPERTY()
	FVector LocationOffset;

	/** Required for file versioning */
	static const FGuid PointCloudFileGUID;
	static const int32 PointCloudFileVersion;

private:
	/** Used for caching the asset registry tag data. */
	struct FLidarPointCloudAssetRegistryCache
	{
		FString PointCount;
		FString ApproxSize;
	} PointCloudAssetRegistryCache;

	/** Contains the list of imported classification IDs */
	UPROPERTY()
	TArray<uint8> ClassificationsImported;

	/** Used for async processing */
	FThreadSafeBool bAsyncCancelled;
	FCriticalSection ProcessingLock;

	/** Notifications we hold on to, that indicate status and progress. */
	class FLidarPointCloudNotificationManager
	{
		TArray<TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe>> Notifications;
		TWeakObjectPtr<ULidarPointCloud> Owner;

	public:
		FLidarPointCloudNotificationManager() : FLidarPointCloudNotificationManager(nullptr) {}
		FLidarPointCloudNotificationManager(TWeakObjectPtr<ULidarPointCloud> Owner) : Owner(Owner) {}
		TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Create(const FString& Text, FThreadSafeBool* bCancelPtr = nullptr, const FString& Icon = "ClassIcon32.LidarPointCloud");
		void CloseAll();
	} Notifications;

	/** Description of collision */
	UPROPERTY(transient, duplicatetransient)
	TObjectPtr<UBodySetup> BodySetup;
	UPROPERTY(transient, duplicatetransient)
	TObjectPtr<UBodySetup> NewBodySetup;

	/** Used for collision building */
	FThreadSafeBool bCollisionBuildInProgress;

	FOnPointCloudChanged OnPointCloudRebuiltEvent;
	FOnPointCloudChanged OnPointCloudUpdateCollisionEvent;
	FOnPointCloudChanged OnPointCloudNormalsUpdatedEvent;
	FOnPointCloudChanged OnPreSaveCleanupEvent;

public:
	ULidarPointCloud();

	FOnPointCloudChanged& OnPointCloudRebuilt() { return OnPointCloudRebuiltEvent; }
	FOnPointCloudChanged& OnPointCloudCollisionUpdated() { return OnPointCloudUpdateCollisionEvent; }
	FOnPointCloudChanged& OnPointCloudNormalsUpdated() { return OnPointCloudNormalsUpdatedEvent; }
	FOnPointCloudChanged& OnPreSaveCleanup() { return OnPreSaveCleanupEvent; }

	// Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	virtual void BeginDestroy() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	virtual void ClearAllCachedCookedPlatformData() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End UObject Interface.

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta=(ScriptName="GetNumLods;GetNumLODs"))
	int32 GetNumLODs() const { return Octree.GetNumLODs(); }

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	int64 GetNumPoints() const { return Octree.GetNumPoints(); }

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	int64 GetNumVisiblePoints() const { return Octree.GetNumVisiblePoints(); }

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	int32 GetNumNodes() const { return Octree.GetNumNodes(); }

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	inline float GetEstimatedPointSpacing() const { return Octree.GetEstimatedPointSpacing(); }

	/** Returns the amount of memory in MB used to store the point cloud. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	int32 GetDataSize() const;

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	FString GetSourcePath() const { return SourcePath.FilePath; }

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool IsOptimizedForDynamicData() const { return bOptimizedForDynamicData; }

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	FBox GetBounds(bool bUseOriginalCoordinates = false) const { return Octree.GetBounds().ShiftBy(bUseOriginalCoordinates ? OriginalCoordinates : LocationOffset); }
	
	/** Recalculates and updates points bounds. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RefreshBounds();

	/** Returns true, if the Octree has collision built */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasCollisionData() const;
	
	/** Returns the number of polygons in the collider or 0 if no collider is built */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	int32 GetColliderPolys() const;

	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RefreshRendering();

	TArray<uint8> GetClassificationsImported() { return ClassificationsImported; }

	/** Returns true if there are any points within the given sphere. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasPointsInSphere(FVector Center, float Radius, bool bVisibleOnly) const
	{
		return HasPointsInSphere(FSphere(Center, Radius), bVisibleOnly);
	}
	bool HasPointsInSphere(const FSphere& Sphere, bool bVisibleOnly) const
	{
		return Octree.HasPointsInSphere(FSphere(Sphere.Center - LocationOffset, Sphere.W), bVisibleOnly);
	}

	/** Returns true if there are any points within the given box. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasPointsInBox(FVector Center, FVector Extent, bool bVisibleOnly) const
	{
		return HasPointsInBox(FBox(Center - Extent, Center + Extent), bVisibleOnly);
	}
	bool HasPointsInBox(const FBox& Box, bool bVisibleOnly) const
	{
		return Octree.HasPointsInBox(Box.ShiftBy(-LocationOffset), bVisibleOnly);
	}

	/** Returns true if there are any points hit by the given ray. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasPointsByRay(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) const
	{
		return HasPointsByRay(FLidarPointCloudRay(Origin, Direction), Radius, bVisibleOnly);
	}
	bool HasPointsByRay(const FLidarPointCloudRay& Ray, float Radius, bool bVisibleOnly) const
	{
		return Octree.HasPointsByRay(Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly);
	}

	/** Populates the given array with points from the tree */
	void GetPoints(TArray<FLidarPointCloudPoint*>& Points, int64 StartIndex = 0, int64 Count = -1);
	void GetPoints(TArray64<FLidarPointCloudPoint*>& Points, int64 StartIndex = 0, int64 Count = -1);

	/** Populates the array with the list of points within the given sphere. */
	void GetPointsInSphere(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly);
	void GetPointsInSphere(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly);

	/** Populates the array with the list of points within the given box. */
	void GetPointsInBox(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly);
	void GetPointsInBox(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly);

	/**
	 * Populates the array with the list of points within the given convex volume.
	 * The volume is assumed to include the LocationOffset of the asset
	 */
	void GetPointsInConvexVolume(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly);
	void GetPointsInConvexVolume(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly);

	/**
	 * Returns an array with copies of points from the tree
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	TArray<FLidarPointCloudPoint> GetPointsAsCopies(bool bReturnWorldSpace, int32 StartIndex = 0, int32 Count = -1) const;
	void GetPointsAsCopies(TArray<FLidarPointCloudPoint>& Points, bool bReturnWorldSpace, int64 StartIndex = 0, int64 Count = -1) const;
	void GetPointsAsCopies(TArray64<FLidarPointCloudPoint>& Points, bool bReturnWorldSpace, int64 StartIndex = 0, int64 Count = -1) const;

	/**
	 * Returns an array with copies of points within the given sphere
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	TArray<FLidarPointCloudPoint> GetPointsInSphereAsCopies(FVector Center, float Radius, bool bVisibleOnly, bool bReturnWorldSpace);
	void GetPointsInSphereAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, bool bReturnWorldSpace) const;
	void GetPointsInSphereAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, bool bReturnWorldSpace) const;

	/**
	 * Returns an array with copies of points within the given box
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	TArray<FLidarPointCloudPoint> GetPointsInBoxAsCopies(FVector Center, FVector Extent, bool bVisibleOnly, bool bReturnWorldSpace);
	void GetPointsInBoxAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, bool bReturnWorldSpace) const;
	void GetPointsInBoxAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, bool bReturnWorldSpace) const;

	/** Performs a raycast test against the point cloud. Returns the pointer if hit or nullptr otherwise. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (Keywords = "raycast"))
	bool LineTraceSingle(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, FLidarPointCloudPoint& PointHit);
	FLidarPointCloudPoint* LineTraceSingle(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly) { return Octree.RaycastSingle(Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/**
	 * Performs a raycast test against the point cloud.
	 * Populates OutHits array with the results.
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 * Returns true it anything has been hit.
	 */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (Keywords = "raycast"))
	bool LineTraceMulti(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, bool bReturnWorldSpace, TArray<FLidarPointCloudPoint>& OutHits) { return LineTraceMulti(FLidarPointCloudRay(Origin, Direction), Radius, bVisibleOnly, bReturnWorldSpace, OutHits); }
	bool LineTraceMulti(const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly, bool bReturnWorldSpace, TArray<FLidarPointCloudPoint>& OutHits)
	{
		FTransform LocalToWorld(LocationOffset);
		return Octree.RaycastMulti(Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr, OutHits);
	}
	bool LineTraceMulti(const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly, TArray<FLidarPointCloudPoint*>& OutHits) { return Octree.RaycastMulti(Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly, OutHits); }

	/** Sets visibility of points within the given sphere. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfPointsInSphere(bool bNewVisibility, FVector Center, float Radius) { SetVisibilityOfPointsInSphere(bNewVisibility, FSphere(Center, Radius)); }
	void SetVisibilityOfPointsInSphere(const bool& bNewVisibility, FSphere Sphere)
	{
		Sphere.Center -= LocationOffset;
		Octree.SetVisibilityOfPointsInSphere(bNewVisibility, Sphere);
	}

	/** Sets visibility of points within the given box. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfPointsInBox(bool bNewVisibility, FVector Center, FVector Extent) { SetVisibilityOfPointsInBox(bNewVisibility, FBox(Center - Extent, Center + Extent)); }
	void SetVisibilityOfPointsInBox(const bool& bNewVisibility, const FBox& Box) { Octree.SetVisibilityOfPointsInBox(bNewVisibility, Box.ShiftBy(-LocationOffset)); }

	/** Sets visibility of the first point hit by the given ray. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfFirstPointByRay(bool bNewVisibility, FVector Origin, FVector Direction, float Radius) { SetVisibilityOfFirstPointByRay(bNewVisibility, FLidarPointCloudRay(Origin, Direction), Radius); }
	void SetVisibilityOfFirstPointByRay(const bool& bNewVisibility, const FLidarPointCloudRay& Ray, const float& Radius) { Octree.SetVisibilityOfFirstPointByRay(bNewVisibility, Ray.ShiftBy(-LocationOffset), Radius); }

	/** Sets visibility of points hit by the given ray. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfPointsByRay(bool bNewVisibility, FVector Origin, FVector Direction, float Radius) { SetVisibilityOfPointsByRay(bNewVisibility, FLidarPointCloudRay(Origin, Direction), Radius); }
	void SetVisibilityOfPointsByRay(const bool& bNewVisibility, const FLidarPointCloudRay& Ray, const float& Radius) { Octree.SetVisibilityOfPointsByRay(bNewVisibility, Ray.ShiftBy(-LocationOffset), Radius); }

	/** Marks all points hidden */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void HideAll() { Octree.HideAll(); }

	/** Marks all points visible */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void UnhideAll() { Octree.UnhideAll(); }

	/** Executes the provided action on each of the points. */
	void ExecuteActionOnAllPoints(TFunction<void(FLidarPointCloudPoint*)> Action, const bool& bVisibleOnly) { Octree.ExecuteActionOnAllPoints(MoveTemp(Action), bVisibleOnly); }

	/** Executes the provided action on each of the points within the given sphere. */
	void ExecuteActionOnPointsInSphere(TFunction<void(FLidarPointCloudPoint*)> Action, const FVector& Center, const float& Radius, const bool& bVisibleOnly) { ExecuteActionOnPointsInSphere(MoveTemp(Action), FSphere(Center, Radius), bVisibleOnly); }
	void ExecuteActionOnPointsInSphere(TFunction<void(FLidarPointCloudPoint*)> Action, FSphere Sphere, const bool& bVisibleOnly)
	{
		Sphere.Center -= LocationOffset;
		Octree.ExecuteActionOnPointsInSphere(MoveTemp(Action), Sphere, bVisibleOnly);
	}
	
	/** Executes the provided action on each of the points within the given box. */
	void ExecuteActionOnPointsInBox(TFunction<void(FLidarPointCloudPoint*)> Action, const FVector& Center, const FVector& Extent, const bool& bVisibleOnly) { ExecuteActionOnPointsInBox(MoveTemp(Action), FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void ExecuteActionOnPointsInBox(TFunction<void(FLidarPointCloudPoint*)> Action, const FBox& Box, const bool& bVisibleOnly) { Octree.ExecuteActionOnPointsInBox(MoveTemp(Action), Box.ShiftBy(-LocationOffset), bVisibleOnly); }

	/** Executes the provided action on the first point hit by the given ray. */
	void ExecuteActionOnFirstPointByRay(TFunction<void(FLidarPointCloudPoint*)> Action, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly) { Octree.ExecuteActionOnFirstPointByRay(MoveTemp(Action), Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/** Executes the provided action on each of the points hit by the given ray. */
	void ExecuteActionOnPointsByRay(TFunction<void(FLidarPointCloudPoint*)> Action, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly) { Octree.ExecuteActionOnPointsByRay(MoveTemp(Action), Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/** Applies the given color to all points */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToAllPoints(const FColor& NewColor, const bool& bVisibleOnly) { Octree.ApplyColorToAllPoints(NewColor, bVisibleOnly); }

	/** Applies the given color to all points within the sphere */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToPointsInSphere(FColor NewColor, FVector Center, float Radius, bool bVisibleOnly) { ApplyColorToPointsInSphere(NewColor, FSphere(Center, Radius), bVisibleOnly); }
	void ApplyColorToPointsInSphere(const FColor& NewColor, FSphere Sphere, const bool& bVisibleOnly)
	{
		Sphere.Center -= LocationOffset;
		Octree.ApplyColorToPointsInSphere(NewColor, Sphere, bVisibleOnly);
	}

	/** Applies the given color to all points within the box */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToPointsInBox(FColor NewColor, FVector Center, FVector Extent, bool bVisibleOnly) { ApplyColorToPointsInBox(NewColor, FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void ApplyColorToPointsInBox(const FColor& NewColor, const FBox& Box, const bool& bVisibleOnly) { Octree.ApplyColorToPointsInBox(NewColor, Box.ShiftBy(-LocationOffset), bVisibleOnly); }

	/** Applies the given color to the first point hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToFirstPointByRay(FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { ApplyColorToFirstPointByRay(NewColor, FLidarPointCloudRay(Origin, Direction), Radius, bVisibleOnly); }
	void ApplyColorToFirstPointByRay(const FColor& NewColor, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly) { Octree.ApplyColorToFirstPointByRay(NewColor, Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/** Applies the given color to all points hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToPointsByRay(FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { ApplyColorToPointsByRay(NewColor, FLidarPointCloudRay(Origin, Direction), Radius, bVisibleOnly); }
	void ApplyColorToPointsByRay(const FColor& NewColor, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly) { Octree.ApplyColorToPointsByRay(NewColor, Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/**
	 * This should to be called if any manual modification to individual points' visibility has been made.
	 * If not marked dirty, the rendering may work sub-optimally.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void MarkPointVisibilityDirty() { Octree.MarkPointVisibilityDirty(); }

#if WITH_EDITOR
	/** Set of editor helper functions */
	void SelectByConvexVolume(FConvexVolume ConvexVolume, bool bAdditive, bool bApplyLocationOffset, bool bVisibleOnly);
	void SelectBySphere(FSphere Sphere, bool bAdditive, bool bApplyLocationOffset, bool bVisibleOnly);
	void HideSelected();
	void DeleteSelected();
	void InvertSelection();
	int64 NumSelectedPoints() const;
	bool HasSelectedPoints() const;
	template <typename T>
	void GetSelectedPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, FTransform Transform) const;
	void GetSelectedPointsAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const;
	void GetSelectedPointsAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const;
	void CalculateNormalsForSelection();
	void ClearSelection();
	void BuildStaticMeshBuffersForSelection(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform);
#endif
	
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetSourcePath(const FString& NewSourcePath);

	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetOptimizedForDynamicData(bool bNewOptimizedForDynamicData);

	/**
	 * Re-initializes the asset with new bounds.
	 * Warning: Will erase all currently held data!
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void Initialize(const FBox& NewBounds)
	{
		LocationOffset = OriginalCoordinates = NewBounds.GetCenter();
		Octree.Initialize((FVector3f)NewBounds.GetExtent());
	}

	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetOptimalCollisionError();

	/** Builds collision mesh for the cloud, using current collision settings */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void BuildCollision() { BuildCollision(nullptr); }
	void BuildCollision(TFunction<void(bool)> CompletionCallback);

	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo"))
	void BuildCollisionWithCallback(UObject* WorldContextObject, FLatentActionInfo LatentInfo, bool& bSuccess);

	/** Removes collision mesh from the cloud. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemoveCollision();

	/** Constructs and returns the MeshBuffers struct from the data */
	void BuildStaticMeshBuffers(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform);
	
	/** Returns true, if the cloud is fully and persistently loaded. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool IsFullyLoaded() const { return Octree.IsFullyLoaded(); }

	/** Persistently loads all nodes. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void LoadAllNodes() { Octree.LoadAllNodes(true); }

	/** Applies given offset to this point cloud. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetLocationOffset(FVector Offset);

	/** Centers this cloud */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void CenterPoints() { SetLocationOffset(FVector::ZeroVector); }

	/** Restores original coordinates */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RestoreOriginalCoordinates() { SetLocationOffset(OriginalCoordinates); }

	/** Returns true, if the cloud has been centered. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	bool IsCentered() const { return LocationOffset.IsNearlyZero(0.1f); }

	/** Re-imports the cloud from it's original source file, overwriting any current point information. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo", ExpandEnumAsExecs = "AsyncMode"))
	void Reimport(UObject* WorldContextObject, bool bUseAsync, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress);
	void Reimport(const FLidarPointCloudAsyncParameters& AsyncParameters);

	/**
	 * Exports this Point Cloud to the given filename.
	 * Consult supported export formats.
	 * Returns true if successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	bool Export(const FString& Filename);

	/**
	 * Inserts the given point into the Octree structure.
	 * If bRefreshPointsBounds is set to false, make sure you call RefreshBounds() manually or cloud centering may not work correctly.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void InsertPoint(const FLidarPointCloudPoint& Point, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation);

	/**
	 * Inserts group of points into the Octree structure, multi-threaded.
	 * If bRefreshPointsBounds is set to false, make sure you call RefreshBounds() manually or cloud centering may not work correctly.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void InsertPoints(const TArray<FLidarPointCloudPoint>& Points, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation) { InsertPoints(Points.GetData(), Points.Num(), DuplicateHandling, bRefreshPointsBounds, Translation); }

	/**
	 * Inserts group of points into the Octree structure, multi-threaded.
	 * If bRefreshPointsBounds is set to false, make sure you call RefreshBounds() manually or cloud centering may not work correctly.
	 * Can be optionally passed a cancellation pointer - if it ever becomes non-null with value of true, process will be canceled.
	 * May also provide progress callback, called approximately every 1% of progress.
	 * Returns false if canceled.
	 */
	bool InsertPoints(FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	bool InsertPoints(const FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	bool InsertPoints(FLidarPointCloudPoint** InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	bool InsertPoints_NoLock(FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	bool InsertPoints_NoLock(const FLidarPointCloudPoint* InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	bool InsertPoints_NoLock(FLidarPointCloudPoint** InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());

	/** Attempts to remove the given point. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePoint(FLidarPointCloudPoint Point)
	{
		FScopeLock Lock(&Octree.DataLock);
		Octree.RemovePoint(Point);
	}
	void RemovePoint_NoLock(FLidarPointCloudPoint Point)
	{
		Octree.RemovePoint(Point);
	}
	void RemovePoint(const FLidarPointCloudPoint* Point)
	{
		FScopeLock Lock(&Octree.DataLock);
		RemovePoint_NoLock(Point);
	}
	void RemovePoint_NoLock(const FLidarPointCloudPoint* Point)
	{
		Octree.RemovePoint(Point);
	}

	/** Removes points in bulk */
	void RemovePoints(TArray<FLidarPointCloudPoint*>& Points)
	{
		FScopeLock Lock(&Octree.DataLock);
		RemovePoints_NoLock(Points);
	}
	void RemovePoints(TArray64<FLidarPointCloudPoint*>& Points)
	{
		FScopeLock Lock(&Octree.DataLock);
		RemovePoints_NoLock(Points);
	}
	void RemovePoints_NoLock(TArray<FLidarPointCloudPoint*>& Points) { Octree.RemovePoints(Points); }
	void RemovePoints_NoLock(TArray64<FLidarPointCloudPoint*>& Points) { Octree.RemovePoints(Points); }

	/** Removes all points within the given sphere  */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePointsInSphere(FVector Center, float Radius, bool bVisibleOnly) { RemovePointsInSphere(FSphere(Center, Radius), bVisibleOnly); }
	void RemovePointsInSphere(FSphere Sphere, const bool& bVisibleOnly)
	{
		Sphere.Center -= LocationOffset;
		Octree.RemovePointsInSphere(Sphere, bVisibleOnly);
	}

	/** Removes all points within the given box */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePointsInBox(FVector Center, FVector Extent, bool bVisibleOnly) { RemovePointsInBox(FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void RemovePointsInBox(const FBox& Box, const bool& bVisibleOnly) { Octree.RemovePointsInBox(Box.ShiftBy(-LocationOffset), bVisibleOnly); }

	/** Removes the first point hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemoveFirstPointByRay(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { RemoveFirstPointByRay(FLidarPointCloudRay(Origin, Direction), Radius, bVisibleOnly); }
	void RemoveFirstPointByRay(const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly) { Octree.RemoveFirstPointByRay(Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/** Removes all points hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePointsByRay(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { RemovePointsByRay(FLidarPointCloudRay(Origin, Direction), Radius, bVisibleOnly); }
	void RemovePointsByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly) { Octree.RemovePointsByRay(Ray.ShiftBy(-LocationOffset), Radius, bVisibleOnly); }

	/** Removes all hidden points */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemoveHiddenPoints() { Octree.RemoveHiddenPoints(); }

	/** Reinitializes the cloud with the new set of data. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	bool SetData(const TArray<FLidarPointCloudPoint>& Points) { return SetData(Points.GetData(), Points.Num()); }
	bool SetData(TArray<FLidarPointCloudPoint*>& Points) { return SetData(Points.GetData(), Points.Num()); }
	bool SetData(const TArray64<FLidarPointCloudPoint>& Points) { return SetData(Points.GetData(), Points.Num()); }
	bool SetData(TArray64<FLidarPointCloudPoint*>& Points) { return SetData(Points.GetData(), Points.Num()); }
	bool SetData(const FLidarPointCloudPoint* Points, const int64& Count, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	bool SetData(FLidarPointCloudPoint** Points, const int64& Count, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());

	/** Merges this point cloud with the ones provided */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void Merge(TArray<ULidarPointCloud*> PointCloudsToMerge) { Merge(PointCloudsToMerge, nullptr); }
	void Merge(TArray<ULidarPointCloud*> PointCloudsToMerge, TFunction<void(void)> ProgressCallback);

	/** Merges this point cloud with the one provided */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void MergeSingle(ULidarPointCloud* PointCloudToMerge) { Merge(TArray<ULidarPointCloud*>({ PointCloudToMerge })); }

	/** Calculates Normals for this point cloud */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (Latent, LatentInfo = "LatentInfo"))
	void CalculateNormals(FLatentActionInfo LatentInfo);

	/**
	 * Calculates Normals for the provided points
	 * If a nullptr is passed as Points, the calculation will be executed on the whole cloud.
	 */
	void CalculateNormals(TArray64<FLidarPointCloudPoint*>* Points, TFunction<void(void)> CompletionCallback);

	//~ Begin Interface_CollisionDataProvider Interface
	virtual bool GetPhysicsTriMeshData(FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override { return HasCollisionData(); }
	virtual bool WantsNegXTriMesh() override { return false; }
	//~ End Interface_CollisionDataProvider Interface

	UBodySetup* GetBodySetup();

	//~ Begin Deprecated
	UE_DEPRECATED(4.27, "Use GetPointsInConvexVolume instead.")
	void GetPointsInFrustum(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly);
	UE_DEPRECATED(4.27, "Use GetPointsInConvexVolume instead.")
	void GetPointsInFrustum(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly);
	//~ End Deprecated

public:
	/** Aligns provided clouds based on the relative offset between their Original Coordinates. Retains overall centering of the group. */
	static void AlignClouds(TArray<ULidarPointCloud*> PointCloudsToAlign);

	/**
	 * Returns new Point Cloud object imported using the settings provided.
	 * Use nullptr as ImportSettings parameter to use default set of settings instead.
	 */
	static ULidarPointCloud* CreateFromFile(const FString& Filename, TSharedPtr<struct FLidarPointCloudImportSettings> ImportSettings = nullptr, UObject* InParent = (UObject*)GetTransientPackage(), FName InName = NAME_None, EObjectFlags Flags = RF_NoFlags)
	{
		return CreateFromFile(Filename, FLidarPointCloudAsyncParameters(GetDefault<ULidarPointCloudSettings>()->bUseAsyncImport), ImportSettings, InParent, InName, Flags);
	}
	static ULidarPointCloud* CreateFromFile(const FString& Filename, const FLidarPointCloudAsyncParameters& AsyncParameters, TSharedPtr<struct FLidarPointCloudImportSettings> ImportSettings = nullptr, UObject* InParent = (UObject*)GetTransientPackage(), FName InName = NAME_None, EObjectFlags Flags = RF_NoFlags);

	/*
	 * Returns new Point Cloud object created from the data provided.
	 * Warning: If using Async, make sure the data does not get invalidated during processing!
	 */
	template<typename T>
	static ULidarPointCloud* CreateFromData(T Points, const int64& Count, const FLidarPointCloudAsyncParameters& AsyncParameters);
	static ULidarPointCloud* CreateFromData(const TArray<FLidarPointCloudPoint>& Points, const bool& bUseAsync);
	static ULidarPointCloud* CreateFromData(const TArray64<FLidarPointCloudPoint>& Points, const bool& bUseAsync);
	static ULidarPointCloud* CreateFromData(const TArray<FLidarPointCloudPoint>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters);
	static ULidarPointCloud* CreateFromData(const TArray64<FLidarPointCloudPoint>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters);
	static ULidarPointCloud* CreateFromData(TArray<FLidarPointCloudPoint*>& Points, const bool& bUseAsync);
	static ULidarPointCloud* CreateFromData(TArray64<FLidarPointCloudPoint*>& Points, const bool& bUseAsync);
	static ULidarPointCloud* CreateFromData(TArray<FLidarPointCloudPoint*>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters);
	static ULidarPointCloud* CreateFromData(TArray64<FLidarPointCloudPoint*>& Points, const FLidarPointCloudAsyncParameters& AsyncParameters);

	/** Returns bounds fitting the given list of points */
	static FBox CalculateBoundsFromPoints(const FLidarPointCloudPoint* Points, const int64& Count);
	static FBox CalculateBoundsFromPoints(FLidarPointCloudPoint** Points, const int64& Count);

private:
	/** Once async physics cook is done, create needed state */
	void FinishPhysicsAsyncCook(TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification) { FinishPhysicsAsyncCook(true, Notification); }
	void FinishPhysicsAsyncCook(bool bSuccess, TSharedRef<FLidarPointCloudNotification, ESPMode::ThreadSafe> Notification);

	void InitializeCollisionRendering();
	void ReleaseCollisionRendering(bool bDestroyAfterRelease);

	template <typename T>
	void GetPoints_Internal(TArray<FLidarPointCloudPoint*, T>& Points, int64 StartIndex = 0, int64 Count = -1);
	template <typename T>
	void GetPointsInSphere_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, FSphere Sphere, const bool& bVisibleOnly);
	template <typename T>
	void GetPointsInBox_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly);
	template <typename T>
	void GetPointsInConvexVolume_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly);
	template <typename T>
	void GetPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& Points, bool bReturnWorldSpace, int64 StartIndex = 0, int64 Count = -1) const;
	template <typename T>
	void GetPointsInSphereAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, FSphere Sphere, const bool& bVisibleOnly, bool bReturnWorldSpace) const;
	template <typename T>
	void GetPointsInBoxAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, bool bReturnWorldSpace) const;

	template<typename T>
	bool InsertPoints_Internal(T InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
	template<typename T>
	bool InsertPoints_NoLock_Internal(T InPoints, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector& Translation, FThreadSafeBool* bCanceled = nullptr, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());

	template<typename T>
	bool SetData_Internal(T Points, const int64& Count, TFunction<void(float)> ProgressCallback = TFunction<void(float)>());
};

USTRUCT(BlueprintType)
struct FLidarPointCloudTraceHit
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collision")
	TObjectPtr<ALidarPointCloudActor> Actor = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collision")
	TObjectPtr<ULidarPointCloudComponent> Component = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collision")
	TArray<FLidarPointCloudPoint> Points;

	FLidarPointCloudTraceHit() {}

	FLidarPointCloudTraceHit(ALidarPointCloudActor* Actor, ULidarPointCloudComponent* Component)
		: Actor(Actor)
		, Component(Component)
	{
	}
};

/**
 * Blueprint library for the Point Cloud assets
 */
UCLASS(BlueprintType)
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Returns new, empty Point Cloud object. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (DisplayName = "Create Empty Lidar Point Cloud"))
	static ULidarPointCloud* CreatePointCloudEmpty() { return NewObject<ULidarPointCloud>(); }

	/**
	 * Returns new Point Cloud object imported using default settings.
	 * If using Async, the process runs in the background without blocking the game thread.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo", ExpandEnumAsExecs = "AsyncMode", DisplayName = "Create Lidar Point Cloud From File"))
	static void CreatePointCloudFromFile(UObject* WorldContextObject, const FString& Filename, bool bUseAsync, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud);
	static void CreatePointCloudFromFile(UObject* WorldContextObject, const FString& Filename, bool bUseAsync, FLatentActionInfo LatentInfo, TSharedPtr<struct FLidarPointCloudImportSettings> ImportSettings, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud);

	/*
	 * Returns new Point Cloud object created from the data provided.
	 * Warning: If using Async, make sure the data does not get invalidated during processing!
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (Latent, WorldContext = "WorldContextObject", LatentInfo = "LatentInfo", ExpandEnumAsExecs = "AsyncMode", DisplayName = "Create Lidar Point Cloud From Data"))
	static void CreatePointCloudFromData(UObject* WorldContextObject, const TArray<FLidarPointCloudPoint>& Points, bool bUseAsync, FLatentActionInfo LatentInfo, ELidarPointCloudAsyncMode& AsyncMode, float& Progress, ULidarPointCloud*& PointCloud);

	/**
	 * Exports the Point Cloud to the given filename.
	 * Consult supported export formats.
	 * Returns true if successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	static bool ExportPointCloudToFile(ULidarPointCloud* PointCloud, const FString& Filename) { return PointCloud && PointCloud->Export(Filename); }

	/** Aligns provided clouds based on the relative offset between their Original Coordinates. Retains overall centering of the group. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	static void AlignClouds(TArray<ULidarPointCloud*> PointCloudsToAlign) { ULidarPointCloud::AlignClouds(PointCloudsToAlign); }

	/** Returns true if there are any points within the given sphere. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static bool ArePointsInSphere(UObject* WorldContextObject, FVector Center, float Radius, bool bVisibleOnly);

	/** Returns true if there are any points within the given box. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static bool ArePointsInBox(UObject* WorldContextObject, FVector Center, FVector Extent, bool bVisibleOnly);

	/** Returns true if there are any points hit by the given ray. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static bool ArePointsByRay(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly);

	/** Returns an array with copies of points within the given sphere */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void GetPointsInSphereAsCopies(UObject* WorldContextObject, TArray<FLidarPointCloudPoint>& SelectedPoints, FVector Center, float Radius, bool bVisibleOnly);

	/** Returns an array with copies of points within the given box */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void GetPointsInBoxAsCopies(UObject* WorldContextObject, TArray<FLidarPointCloudPoint>& SelectedPoints, FVector Center, FVector Extent, const bool& bVisibleOnly);

	/** Does a collision trace along the given line and returns the first blocking hit encountered. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject", DisplayName = "LineTraceForLidarPointCloud", Keywords = "raycast"))
	static bool LineTraceSingle(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, FLidarPointCloudTraceHit& Hit);

	/** Does a collision trace along the given line and returns all hits encountered up to and including the first blocking hit. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject", DisplayName = "LineTraceMultiForLidarPointCloud", Keywords = "raycast"))
	static bool LineTraceMulti(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, TArray<FLidarPointCloudTraceHit>& Hits);

	/** Sets visibility of points within the given sphere. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void SetVisibilityOfPointsInSphere(UObject* WorldContextObject, bool bNewVisibility, FVector Center, float Radius);

	/** Sets visibility of points within the given box. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void SetVisibilityOfPointsInBox(UObject* WorldContextObject, bool bNewVisibility, FVector Center, FVector Extent);

	/** Sets visibility of the first point hit by the given ray. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void SetVisibilityOfFirstPointByRay(UObject* WorldContextObject, bool bNewVisibility, FVector Origin, FVector Direction, float Radius);

	/** Sets visibility of points hit by the given ray. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void SetVisibilityOfPointsByRay(UObject* WorldContextObject, bool bNewVisibility, FVector Origin, FVector Direction, float Radius);

	/** Applies the given color to all points within the sphere */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void ApplyColorToPointsInSphere(UObject* WorldContextObject, FColor NewColor, FVector Center, float Radius, bool bVisibleOnly);

	/** Applies the given color to all points within the box */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void ApplyColorToPointsInBox(UObject* WorldContextObject, FColor NewColor, FVector Center, FVector Extent, bool bVisibleOnly);

	/** Applies the given color to the first point hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void ApplyColorToFirstPointByRay(UObject* WorldContextObject, FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly);

	/** Applies the given color to all points hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void ApplyColorToPointsByRay(UObject* WorldContextObject, FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly);

	/** Removes all points within the given sphere  */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void RemovePointsInSphere(UObject* WorldContextObject, FVector Center, float Radius, bool bVisibleOnly);

	/** Removes all points within the given box */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void RemovePointsInBox(UObject* WorldContextObject, FVector Center, FVector Extent, bool bVisibleOnly);

	/** Removes the first point hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void RemoveFirstPointByRay(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly);

	/** Removes all points hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud", meta = (WorldContext = "WorldContextObject"))
	static void RemovePointsByRay(UObject* WorldContextObject, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly);

	/** Sets the given normal using provided vector */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	static void NormalFromVector(UPARAM(ref) FLidarPointCloudNormal& Normal, FVector Vector) { Normal.SetFromVector((FVector3f)Vector); }

	/** Converts a Lidar Point Cloud Normal to a Vector */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Normal to Vector", CompactNodeTitle = "->", BlueprintAutocast), Category = "Lidar Point Cloud")
	static FVector Conv_LidarPointCloudNormalToVector(const FLidarPointCloudNormal& Normal) { return (FVector)Normal.ToVector(); }

	/** Converts a Vector to a Lidar Point Cloud Normal */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Vector to Normal", CompactNodeTitle = "->", BlueprintAutocast), Category = "Lidar Point Cloud")
	static FLidarPointCloudNormal Conv_VectorToLidarPointCloudNormal(const FVector& Vector) { return FLidarPointCloudNormal((FVector3f)Vector); }
};

UCLASS(hidecategories = (Collision, Brush, Attachment, Physics, Volume, BrushBuilder), MinimalAPI)
class ALidarClippingVolume : public AVolume
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Clipping Volume")
	bool bEnabled;

	/** Affects how this volume affects points */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Clipping Volume")
	ELidarClippingVolumeMode Mode;

	/**
	 * Determines the processing order of the nodes, in case they overlap.
	 * Higher values take priority over lower ones.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Clipping Volume")
	int32 Priority;

	ALidarClippingVolume();
};
