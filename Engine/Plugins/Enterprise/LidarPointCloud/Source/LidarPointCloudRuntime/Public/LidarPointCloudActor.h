// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "LidarPointCloudActor.generated.h"

class ULidarPointCloud;
class ULidarPointCloudComponent;

UCLASS(BlueprintType, hidecategories = ("Materials"))
class LIDARPOINTCLOUDRUNTIME_API ALidarPointCloudActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = PointCloudActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Rendering,Components|LidarPointCloud", AllowPrivateAccess = "true"))
	TObjectPtr<ULidarPointCloudComponent> PointCloudComponent;

public:
#if WITH_EDITOR
	virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif

public:
	/** Returns PointCloudComponent subobject **/
	ULidarPointCloudComponent* GetPointCloudComponent() const { return PointCloudComponent; }

	UFUNCTION(BlueprintPure, Category = "Components|LidarPointCloud")
	ULidarPointCloud* GetPointCloud() const;

	UFUNCTION(BlueprintCallable, Category = "Components|LidarPointCloud")
	void SetPointCloud(ULidarPointCloud *InPointCloud);
};