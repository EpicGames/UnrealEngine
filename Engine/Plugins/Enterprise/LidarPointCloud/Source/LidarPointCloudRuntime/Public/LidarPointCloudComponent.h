// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloud.h"
#include "Components/MeshComponent.h"
#include "LidarPointCloudComponent.generated.h"

class UBodySetup;
class FViewportClient;

UENUM(BlueprintType)
enum class ELidarPointCloudSpriteOrientation : uint8
{
	/** The sprites will face camera, even if Normals are available. */
	PreferFacingCamera,
	/** The sprites will attempt to face Normals, if available, or fall back to facing camera otherwise. */
	PreferFacingNormal,
};

/** Component that allows you to render specified point cloud section */
UCLASS(ClassGroup=Rendering, ShowCategories = (Rendering), HideCategories = (Object, LOD, Physics, Activation, Materials, Cooking, Input, HLOD, Mobile), meta = (BlueprintSpawnableComponent))
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudComponent : public UMeshComponent
{
	GENERATED_BODY()
		
private:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Lidar Point Cloud", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<ULidarPointCloud> PointCloud;

	/**
	 * Allows using custom-built material for the point cloud.
	 * Set to None to use the default one instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Material", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> CustomMaterial;

public:
	/**
	 * Use to tweak the size of the points.
	 * Set to 0 to switch to 1 pixel points.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (ClampMin = "0.0"))
	float PointSize;

	/** Determines how the points will be scaled  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	ELidarPointCloudScalingMethod ScalingMethod;

	/**
	 * If set to > 0, it attempts to close gaps between points.
	 * Setting this too high may cause visual artifacts.
	 * This setting may interfere with AO
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (ClampMin = "0.0"))
	float GapFillingStrength;

	/** Specifies which source to use for point colors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	ELidarPointCloudColorationMode ColorSource;

private:
	/** Affects the shape of points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", BlueprintSetter = SetPointShape, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage="Use GetPointShape() / SetPointShape() instead."))
	ELidarPointCloudSpriteShape PointShape;

public:
	/** Affects the orientation of points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	ELidarPointCloudSpriteOrientation PointOrientation;

	/**
	 * Used with the Classification source.
	 * Maps the given classification ID to a color.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TMap<int32, FLinearColor> ClassificationColors;

	/** Specifies the bottom color of the elevation-based gradient. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Appearance")
	FLinearColor ElevationColorBottom;

	/** Specifies the top color of the elevation-based gradient. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Appearance")
	FLinearColor ElevationColorTop;

	/**
	 * Larger values will help mask LOD transition areas, but too large values will lead to loss of detail.
	 * Values in range 0.035 - 0.05 seem to produce best overall results.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Appearance", meta = (ClampMin = "0.0", ClampMax = "0.15"))
	float PointSizeBias;

	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "saturation"))
	FVector4 Saturation;

	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "contrast"))
	FVector4 Contrast;

	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", ColorGradingMode = "gamma"))
	FVector4 Gamma;

	/** Affects the emissive strength of the color. Useful to create Bloom and light bleed effects. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment", meta = (UIMin = "0.0", UIMax = "1.0", Delta = "0.01", ColorGradingMode = "gain"))
	FVector4 Gain;

	/** Applied additively, 0 being neutral. */
	UPROPERTY(interp, EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", ColorGradingMode = "offset", SupportDynamicSliderMaxValue = "true", SupportDynamicSliderMinValue = "true"))
	FVector4 Offset;

	/** Specifies the tint to apply to the points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment")
	FLinearColor ColorTint;

	/** Specifies the influence of Intensity data, if available, on the overall color. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Adjustment", meta = (ClampMin = "0", ClampMax = "1"))
	float IntensityInfluence;

	/**
	 * If enabled, points outside of the visible frustum will not be rendered.
	 * While most project should leave this enabled, disabling it may help
	 * with the data streaming lag when shooting cinematics.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bUseFrustumCulling;

	/**
	 * Minimum Depth from which the nodes should be rendered.
	 * 0 to disable.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Rendering", meta = (ClampMin = "0"))
	int32 MinDepth;

	/**
	 * Maximum Depth to which the nodes should be rendered.
	 * -1 to disable.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Rendering")
	int32 MaxDepth;

	/** Enabling this will cause the visible nodes to render their bounds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Rendering")
	bool bDrawNodeBounds;

private:
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> Material;
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> BaseMaterial;
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> BaseMaterialMasked;

	/** Pointer to the viewport client of the owning editor, or null, if this is a game object. */
	TWeakPtr<FViewportClient> OwningViewportClient;

public:
	ULidarPointCloudComponent();

	UFUNCTION(BlueprintPure, Category = "Components|LidarPointCloud")
	ULidarPointCloud* GetPointCloud() const { return PointCloud; }

	inline TWeakPtr<FViewportClient> GetOwningViewportClient() const { return OwningViewportClient; }
	inline bool IsOwnedByEditor() const { return OwningViewportClient.IsValid(); }

	/** Returns true if there are any points within the given sphere. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasPointsInSphere(FVector Center, float Radius, bool bVisibleOnly) const
	{
		return HasPointsInSphere(FSphere(Center, Radius), bVisibleOnly);
	}
	bool HasPointsInSphere(const FSphere& Sphere, bool bVisibleOnly) const
	{
		return PointCloud && PointCloud->HasPointsInSphere(Sphere.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
	}

	/** Returns true if there are any points within the given box. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasPointsInBox(FVector Center, FVector Extent, bool bVisibleOnly) const
	{
		return HasPointsInBox(FBox(Center - Extent, Center + Extent), bVisibleOnly);
	}
	bool HasPointsInBox(const FBox& Box, bool bVisibleOnly) const
	{
		return PointCloud && PointCloud->HasPointsInBox(Box.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
	}

	/** Returns true if there are any points hit by the given ray. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	bool HasPointsByRay(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) const
	{
		return HasPointsByRay(FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly);
	}
	bool HasPointsByRay(const FLidarPointCloudRay& Ray, float Radius, bool bVisibleOnly) const
	{
		return PointCloud && PointCloud->HasPointsByRay(Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
	}

	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud")
	TArray<FLidarPointCloudPoint> GetSelectedPointsAsCopies(bool bReturnWorldSpace) const
	{
		TArray<FLidarPointCloudPoint> Points;
#if WITH_EDITOR
		GetSelectedPointsAsCopies(Points, bReturnWorldSpace);
#endif
		return Points;
	}
#if WITH_EDITOR
	void GetSelectedPointsAsCopies(TArray<FLidarPointCloudPoint>& Points, bool bReturnWorldSpace) const
	{
		if(PointCloud)
		{
			PointCloud->GetSelectedPointsAsCopies(Points, bReturnWorldSpace ? GetComponentTransform() : FTransform());
		}
	}
	void GetSelectedPointsAsCopies(TArray64<FLidarPointCloudPoint>& Points, bool bReturnWorldSpace) const
	{
		if(PointCloud)
		{
			PointCloud->GetSelectedPointsAsCopies(Points, bReturnWorldSpace ? GetComponentTransform() : FTransform());
		}
	}
#endif
	
	/** Populates the array with the list of points within the given sphere. */
	void GetPointsInSphere(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FVector& Center, const float& Radius, const bool& bVisibleOnly) { GetPointsInSphere(SelectedPoints, FSphere(Center, Radius), bVisibleOnly); }
	void GetPointsInSphere(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FVector& Center, const float& Radius, const bool& bVisibleOnly) { GetPointsInSphere(SelectedPoints, FSphere(Center, Radius), bVisibleOnly); }
	void GetPointsInSphere(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->GetPointsInSphere(SelectedPoints, Sphere.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}
	void GetPointsInSphere(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->GetPointsInSphere(SelectedPoints, Sphere.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Populates the array with the list of points within the given box. */
	void GetPointsInBox(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FVector& Center, const FVector& Extent, const bool& bVisibleOnly) { GetPointsInBox(SelectedPoints, FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void GetPointsInBox(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FVector& Center, const FVector& Extent, const bool& bVisibleOnly) { GetPointsInBox(SelectedPoints, FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void GetPointsInBox(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->GetPointsInBox(SelectedPoints, Box.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}
	void GetPointsInBox(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->GetPointsInBox(SelectedPoints, Box.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/**
	 * Populates the array with copies of points within the given sphere.
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	TArray<FLidarPointCloudPoint> GetPointsInSphereAsCopies(FVector Center, float Radius, bool bVisibleOnly, bool bReturnWorldSpace)
	{
		TArray<FLidarPointCloudPoint> Points;
		GetPointsInSphereAsCopies(Points, FSphere(Center, Radius), bVisibleOnly, bReturnWorldSpace);
		return Points;
	}

	void GetPointsInSphereAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const bool& bReturnWorldSpace)
	{
		if (PointCloud)
		{
			FTransform LocalToWorld = GetLocalToWorld();
			PointCloud->Octree.GetPointsInSphereAsCopies(SelectedPoints, Sphere.TransformBy(LocalToWorld.Inverse()), bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr);
		}
	}
	void GetPointsInSphereAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const bool& bReturnWorldSpace)
	{
		if (PointCloud)
		{
			FTransform LocalToWorld = GetLocalToWorld();
			PointCloud->Octree.GetPointsInSphereAsCopies(SelectedPoints, Sphere.TransformBy(LocalToWorld.Inverse()), bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr);
		}
	}

	/**
	 * Populates the array with copies of points within the given box.
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	TArray<FLidarPointCloudPoint> GetPointsInBoxAsCopies(FVector Center, FVector Extent, bool bVisibleOnly, bool bReturnWorldSpace)
	{
		TArray<FLidarPointCloudPoint> Points;
		GetPointsInBoxAsCopies(Points, FBox(Center - Extent, Center + Extent), bVisibleOnly, bReturnWorldSpace);
		return Points;
	}
	void GetPointsInBoxAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const bool& bReturnWorldSpace)
	{
		if (PointCloud)
		{
			FTransform LocalToWorld = GetLocalToWorld();
			PointCloud->Octree.GetPointsInBoxAsCopies(SelectedPoints, Box.TransformBy(LocalToWorld.Inverse()), bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr);
		}
	}
	void GetPointsInBoxAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const bool& bReturnWorldSpace)
	{
		if (PointCloud)
		{
			FTransform LocalToWorld = GetLocalToWorld();
			PointCloud->Octree.GetPointsInBoxAsCopies(SelectedPoints, Box.TransformBy(LocalToWorld.Inverse()), bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr);
		}
	}

	/** Performs a raycast test against the point cloud. Returns the pointer if hit or nullptr otherwise. */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (Keywords = "raycast"))
	bool LineTraceSingle(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, FLidarPointCloudPoint& PointHit)
	{
		FLidarPointCloudPoint* Point = LineTraceSingle(FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly);
		if (Point)
		{
			PointHit = *Point;
			return true;
		}

		return false;
	}
	FLidarPointCloudPoint* LineTraceSingle(FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly)
	{
		return PointCloud ? PointCloud->LineTraceSingle(Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly) : nullptr;
	}

	/**
	 * Performs a raycast test against the point cloud.
	 * Populates OutHits array with the results.
	 * If ReturnWorldSpace is selected, the points' locations will be converted into absolute value, otherwise they will be relative to the center of the cloud.
	 * Returns true it anything has been hit.
	 */
	UFUNCTION(BlueprintPure, Category = "Lidar Point Cloud", meta = (Keywords = "raycast"))
	bool LineTraceMulti(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly, bool bReturnWorldSpace, TArray<FLidarPointCloudPoint>& OutHits)
	{
		return LineTraceMulti(FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction).TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly, bReturnWorldSpace, OutHits);
	}
	bool LineTraceMulti(FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly, bool bReturnWorldSpace, TArray<FLidarPointCloudPoint>& OutHits)
	{
		if (PointCloud)
		{
			FTransform LocalToWorld = GetLocalToWorld();
			return PointCloud->Octree.RaycastMulti(Ray.TransformBy(LocalToWorld.Inverse()), Radius, bVisibleOnly, bReturnWorldSpace ? &LocalToWorld : nullptr, OutHits);
		}

		return false;
	}
	bool LineTraceMulti(FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly, TArray<FLidarPointCloudPoint*>& OutHits)
	{
		return PointCloud ? PointCloud->LineTraceMulti(Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly, OutHits) : false;
	}

	/** Sets visibility of points within the given sphere. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfPointsInSphere(bool bNewVisibility, FVector Center, float Radius) { SetVisibilityOfPointsInSphere(bNewVisibility, FSphere(Center, Radius)); }
	void SetVisibilityOfPointsInSphere(const bool& bNewVisibility, const FSphere& Sphere)
	{
		if (PointCloud)
		{
			PointCloud->SetVisibilityOfPointsInSphere(bNewVisibility, Sphere.TransformBy(GetComponentTransform().Inverse()));
		}
	}

	/** Sets visibility of points within the given box. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfPointsInBox(bool bNewVisibility, FVector Center, FVector Extent) { SetVisibilityOfPointsInBox(bNewVisibility, FBox(Center - Extent, Center + Extent)); }
	void SetVisibilityOfPointsInBox(const bool& bNewVisibility, const FBox& Box)
	{
		if (PointCloud)
		{
			PointCloud->SetVisibilityOfPointsInBox(bNewVisibility, Box.TransformBy(GetComponentTransform().Inverse()));
		}
	}

	/** Sets visibility of the first point hit by the given ray. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfFirstPointByRay(bool bNewVisibility, FVector Origin, FVector Direction, float Radius) { SetVisibilityOfFirstPointByRay(bNewVisibility, FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius); }
	void SetVisibilityOfFirstPointByRay(bool bNewVisibility, FLidarPointCloudRay Ray, float Radius)
	{
		if (PointCloud)
		{
			PointCloud->SetVisibilityOfFirstPointByRay(bNewVisibility, Ray.TransformBy(GetComponentTransform().Inverse()), Radius);
		}
	}

	/** Sets visibility of points hit by the given ray. */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void SetVisibilityOfPointsByRay(bool bNewVisibility, FVector Origin, FVector Direction, float Radius) { SetVisibilityOfPointsByRay(bNewVisibility, FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius); }
	void SetVisibilityOfPointsByRay(bool bNewVisibility, FLidarPointCloudRay Ray, float Radius)
	{
		if (PointCloud)
		{
			PointCloud->SetVisibilityOfPointsByRay(bNewVisibility, Ray.TransformBy(GetComponentTransform().Inverse()), Radius);
		}
	}

	/** Executes the provided action on each of the points within the given sphere. */
	void ExecuteActionOnPointsInSphere(TFunction<void(FLidarPointCloudPoint*)> Action, const FVector& Center, const float& Radius, const bool& bVisibleOnly) { ExecuteActionOnPointsInSphere(MoveTemp(Action), FSphere(Center, Radius), bVisibleOnly); }
	void ExecuteActionOnPointsInSphere(TFunction<void(FLidarPointCloudPoint*)> Action, const FSphere& Sphere, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ExecuteActionOnPointsInSphere(MoveTemp(Action), Sphere.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Executes the provided action on each of the points within the given box. */
	void ExecuteActionOnPointsInBox(TFunction<void(FLidarPointCloudPoint*)> Action, const FVector& Center, const FVector& Extent, const bool& bVisibleOnly) { ExecuteActionOnPointsInBox(MoveTemp(Action), FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void ExecuteActionOnPointsInBox(TFunction<void(FLidarPointCloudPoint*)> Action, const FBox& Box, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ExecuteActionOnPointsInBox(MoveTemp(Action), Box.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Executes the provided action on the first point hit by the given ray. */
	void ExecuteActionOnFirstPointByRay(TFunction<void(FLidarPointCloudPoint*)> Action, FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ExecuteActionOnFirstPointByRay(MoveTemp(Action), Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
		}
	}

	/** Executes the provided action on each of the points hit by the given ray. */
	void ExecuteActionOnPointsByRay(TFunction<void(FLidarPointCloudPoint*)> Action, FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ExecuteActionOnPointsByRay(MoveTemp(Action), Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
		}
	}

	/** Applies the given color to all points within the sphere */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToPointsInSphere(FColor NewColor, FVector Center, float Radius, bool bVisibleOnly) { ApplyColorToPointsInSphere(NewColor, FSphere(Center, Radius), bVisibleOnly); }
	void ApplyColorToPointsInSphere(const FColor& NewColor, const FSphere& Sphere, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ApplyColorToPointsInSphere(NewColor, Sphere.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Applies the given color to all points within the box */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToPointsInBox(FColor NewColor, FVector Center, FVector Extent, bool bVisibleOnly) { ApplyColorToPointsInBox(NewColor, FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void ApplyColorToPointsInBox(const FColor& NewColor, const FBox& Box, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ApplyColorToPointsInBox(NewColor, Box.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Applies the given color to the first point hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToFirstPointByRay(FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { ApplyColorToFirstPointByRay(NewColor, FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly); }
	void ApplyColorToFirstPointByRay(const FColor& NewColor, FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ApplyColorToFirstPointByRay(NewColor, Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
		}
	}

	/** Applies the given color to all points hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void ApplyColorToPointsByRay(FColor NewColor, FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { ApplyColorToPointsByRay(NewColor, FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly); }
	void ApplyColorToPointsByRay(const FColor& NewColor, FLidarPointCloudRay Ray, float Radius, bool bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->ApplyColorToPointsByRay(NewColor, Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
		}
	}
	
	/** Removes all points within the given sphere  */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePointsInSphere(FVector Center, float Radius, bool bVisibleOnly) { RemovePointsInSphere(FSphere(Center, Radius), bVisibleOnly); }
	void RemovePointsInSphere(const FSphere& Sphere, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->RemovePointsInSphere(Sphere.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Removes all points within the given box  */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePointsInBox(FVector Center, FVector Extent, bool bVisibleOnly) { RemovePointsInBox(FBox(Center - Extent, Center + Extent), bVisibleOnly); }
	void RemovePointsInBox(const FBox& Box, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->RemovePointsInBox(Box.TransformBy(GetComponentTransform().Inverse()), bVisibleOnly);
		}
	}

	/** Removes the first point hit by the given ray */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemoveFirstPointByRay(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { RemoveFirstPointByRay(FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly); }
	void RemoveFirstPointByRay(const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->RemoveFirstPointByRay(Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
		}
	}

	/** Removes all points hit by the given ray  */
	UFUNCTION(BlueprintCallable, Category = "Lidar Point Cloud")
	void RemovePointsByRay(FVector Origin, FVector Direction, float Radius, bool bVisibleOnly) { RemovePointsByRay(FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction), Radius, bVisibleOnly); }
	void RemovePointsByRay(FLidarPointCloudRay Ray, float Radius, const bool& bVisibleOnly)
	{
		if (PointCloud)
		{
			PointCloud->RemovePointsByRay(Ray.TransformBy(GetComponentTransform().Inverse()), Radius, bVisibleOnly);
		}
	}

#if WITH_EDITOR
	void SelectByConvexVolume(FConvexVolume ConvexVolume, bool bAdditive, bool bVisibleOnly);
	void SelectBySphere(FSphere Sphere, bool bAdditive, bool bVisibleOnly);
	void HideSelected();
	void DeleteSelected();
	void InvertSelection();
	int64 NumSelectedPoints();
	void ClearSelection();
#endif
	
public:
	UFUNCTION(BlueprintCallable, Category = "Components|LidarPointCloud")
	void SetPointCloud(ULidarPointCloud *InPointCloud);

	/** Returns the current Point Shape */
	UFUNCTION(BlueprintPure, Category = "Components|LidarPointCloud")
	inline ELidarPointCloudSpriteShape GetPointShape() const { return PointShape; }

	/** Sets new Point Shape */
	UFUNCTION(BlueprintCallable, Category = "Components|LidarPointCloud")
	void SetPointShape(ELidarPointCloudSpriteShape NewPointShape);

	/** Applies specified rendering parameters (Brightness, Saturation, etc) to the selected material */
	UFUNCTION(BlueprintCallable, Category = "Components|LidarPointCloud|Rendering")
	void ApplyRenderingParameters();

public:
	// Begin UObject Interface.
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PostLoad() override;
	// End UObject Interface.

	// End UMeshComponent Interface
	virtual int32 GetNumMaterials() const override { return 1; }
	virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override { return Material; }
	virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* InMaterial) override;
	// End UMeshComponent Interface

#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditImport() override
	{
		Super::PostEditImport();

		// Make sure to update the material after duplicating this component
		UpdateMaterial();
	}
#endif

	virtual UBodySetup* GetBodySetup() override;

	/** Returns true if the component should be rendered facing normals */
	bool ShouldRenderFacingNormals() const { return PointOrientation == ELidarPointCloudSpriteOrientation::PreferFacingNormal && PointSize > 0; }

private:
	// Begin UPrimitiveComponent Interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	// End UMeshComponent Interface

	// Begin USceneComponent Interface
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	// End USceneComponent Interface

	FTransform GetLocalToWorld()
	{
		FTransform LocalToWorld = GetComponentTransform();
		if (PointCloud)
		{
			LocalToWorld.AddToTranslation(PointCloud->LocationOffset);
		}
		return LocalToWorld;
	}

	void UpdateMaterial();

	void AttachPointCloudListener();
	void RemovePointCloudListener();
	void OnPointCloudRebuilt();
	void OnPointCloudCollisionUpdated();
	void OnPointCloudNormalsUpdated();

	void PostPointCloudSet();

	friend class FPointCloudSceneProxy;
#if WITH_EDITOR
	friend class SLidarPointCloudEditorViewport;
#endif
};
