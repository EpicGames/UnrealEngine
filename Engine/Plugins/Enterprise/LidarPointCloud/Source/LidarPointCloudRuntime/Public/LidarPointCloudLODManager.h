// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConvexVolume.h"
#include "CoreMinimal.h"
#include "Tickable.h"
#include "PrimitiveSceneProxy.h"
#include "LidarPointCloudShared.h"
#include "Rendering/LidarPointCloudRendering.h"

class ULidarPointCloud;
class ULidarPointCloudComponent;
class FLidarPointCloudOctree;
struct FLidarPointCloudTraversalOctree;
struct FLidarPointCloudTraversalOctreeNode;

/** Stores View data required to calculate LODs for Lidar Point Clouds */
struct FLidarPointCloudViewData
{
	bool bValid;
	FVector ViewOrigin;
	FVector ViewDirection;
	float ScreenSizeFactor;
	FConvexVolume ViewFrustum;
	bool bSkipMinScreenSize;
	bool bPIE;
	bool bHasFocus;

	FLidarPointCloudViewData(bool bCompute = false);

	void Compute();
	bool ComputeFromEditorViewportClient(class FViewportClient* ViewportClient);
};

/**  
 * This class is responsible for selecting nodes for rendering among all instances of all LidarPointCloud assets.
 * 
 */
class FLidarPointCloudLODManager : public FTickableGameObject
{
	struct FRegisteredProxy
	{
		TWeakObjectPtr<ULidarPointCloudComponent> Component;
		TWeakObjectPtr<ULidarPointCloud> PointCloud;
		FLidarPointCloudOctree* Octree;
		FLidarPointCloudComponentRenderParams ComponentRenderParams;
		TWeakPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper;
		TSharedPtr<FLidarPointCloudTraversalOctree, ESPMode::ThreadSafe> TraversalOctree;

		FRegisteredProxy(TWeakObjectPtr<ULidarPointCloudComponent> Component, TWeakPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper);

		/** Used to detect transform changes without the need of callbacks from the SceneProxy */
		FTransform LastComponentTransform;

		FLidarPointCloudViewData ViewData;

		/** If true, this proxy will be skipped. Used to avoid point duplication in PIE */
		bool bSkip;
	};

	/** This holds the list of currently registered proxies, which will be used for node selection */
	TArray<FRegisteredProxy> RegisteredProxies;

	/** Allows skipping processing, if another one is already in progress */
	TAtomic<bool> bProcessing;

	/** Stores cumulative time, elapsed from the creation of the manager. Used to determine nodes' lifetime. */
	float Time;

	/** Stores the number of points in visible frustum during last frame */
	FThreadSafeCounter64 NumPointsInFrustum;

	/** Stores the last calculated point budget */
	uint32 LastPointBudget;

public:
	FLidarPointCloudLODManager();

	virtual void Tick(float DeltaTime) override;

	virtual TStatId GetStatId() const override;

	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool IsTickableInEditor() const override { return true; }

	static void RegisterProxy(ULidarPointCloudComponent* Component, TWeakPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper);

	static void RefreshLOD();

private:
	/**
	 * This function:
	 * - Resizes the global IndexBuffer and StructuredBuffer to fit the required GlobalPointBudget
	 * - Iterates over all registered proxies and selects the best set of nodes within the point budget
	 * - Generates the data for the StructuredBuffer and passes it to the Render Thread for an update
	 *
	 * Returns the number of points in visible frustum
	 */
	int64 ProcessLOD(const TArray<FRegisteredProxy>& RegisteredProxies, const float CurrentTime, const uint32 PointBudget, const TArray<FLidarPointCloudClippingVolumeParams>& ClippingVolumes);

	/** Forces a refresh of the LOD processing. Must be called from a GT */
	void ForceProcessLOD();

	/** Called to prepare the proxies for processing */
	void PrepareProxies();

	/** Compiles a list of all clipping volumes affecting any of the registered proxies */
	TArray<FLidarPointCloudClippingVolumeParams> GetClippingVolumes() const;

	static FLidarPointCloudLODManager& Get();
};