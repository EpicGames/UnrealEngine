// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"
#include "LidarPointCloudSettings.h"
#include "Meshing/LidarPointCloudMeshing.h"
#include "HAL/ThreadSafeCounter64.h"
#include "Misc/ScopeLock.h"
#include "Containers/Queue.h"
#include "ConvexVolume.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Serialization/BulkData.h"

class ULidarPointCloud;
class FLidarPointCloudOctree;
struct FLidarPointCloudTraversalOctree;
struct FLidarPointCloudTraversalOctreeNode;
class FLidarPointCloudRenderBuffer;
class FLidarPointCloudVertexFactory;
class FLidarPointCloudRayTracingGeometry;

namespace LidarPointCloudMeshing
{
	struct FMeshBuffers;
};

/**
 * WARNING: Exercise caution when modifying the contents of the Octree, as it may be in use by the Rendering Thread via FPointCloudSceneProxy
 * Use the FLidarPointCloudOctree::DataLock prior to such attempt
 */

/**
 * Child ordering
 * 0	X- Y- Z-
 * 1	X- Y- Z+
 * 2	X- Y+ Z-
 * 3	X- Y+ Z+
 * 4	X+ Y- Z-
 * 5	X+ Y- Z+
 * 6	X+ Y+ Z-
 * 7	X+ Y+ Z+
 */

/**
 * Represents a single octant in the tree.
 */
struct FLidarPointCloudOctreeNode
{
private:
	/** Stores the time, at which the BulkData needs to be released */
	float BulkDataLifetime;

	/** Depth of this node */
	uint8 Depth;

	/** Location of this node inside the parent node - see the Child Ordering at the top of the file */
	uint8 LocationInParent;

	/** Center point of this node. */
	FVector3f Center;

	/** Stores the children array */
	// #todo: Change to TIndirectArray<> - investigate increased memory consumption, ~130 bytes / Node
	TArray<FLidarPointCloudOctreeNode*> Children;

	/** Pointer to the Tree holding this node */
	FLidarPointCloudOctree* Tree;

	/** Marks the node for visibility recalculation next time it's necessary */
	bool bVisibilityDirty;

	/** Marks the node as being used for rendering */
	bool bInUse;

	/** Marks the node as containing active selection */
	bool bHasSelection;

	/** Stores the number of visible points */
	uint32 NumVisiblePoints;

	FCriticalSection MapLock;

	/**
	 * Holds point data allocated to this node
	 * Can be empty, if the data hasn't been streamed in yet
	 */
	TArray<FLidarPointCloudPoint> Data;

	/**
	 * Stores the number of points this node contains.
	 * Needed, since Data may not have been streamed yet, and would return a count of 0.
	 */
	uint32 NumPoints;

	/** True, if the node has its data loaded */
	TAtomic<bool> bHasData;

	/** Offset in the archive file, where the data for this node is located */
	int64 BulkDataOffset;

	uint32 BulkDataSize;

	/** Holds render data for this node */
	TSharedPtr<FLidarPointCloudRenderBuffer> DataCache;
	TSharedPtr<FLidarPointCloudVertexFactory> VertexFactory;
	TSharedPtr<FLidarPointCloudRayTracingGeometry> RayTracingGeometry;
	
	bool bRenderDataDirty;

	/** Used to keep track, which data is available for rendering */
	TAtomic<bool> bHasDataPending;

	/** This is used to prevent nodes with changed content from being overwritten by consecutive streaming. */
	TAtomic<bool> bCanReleaseData;

public:
	inline FLidarPointCloudOctreeNode(FLidarPointCloudOctree* Tree, const uint8& Depth) : FLidarPointCloudOctreeNode(Tree, Depth, 0, FVector3f::ZeroVector) {}
	FLidarPointCloudOctreeNode(FLidarPointCloudOctree* Tree, const uint8& Depth, const uint8& LocationInParent, const FVector3f& Center);
	~FLidarPointCloudOctreeNode();
	FLidarPointCloudOctreeNode(const FLidarPointCloudOctreeNode&) = delete;
	FLidarPointCloudOctreeNode(FLidarPointCloudOctreeNode&&) = delete;
	FLidarPointCloudOctreeNode& operator=(const FLidarPointCloudOctreeNode&) = delete;
	FLidarPointCloudOctreeNode& operator=(FLidarPointCloudOctreeNode&&) = delete;

	/** Returns a pointer to the point data */
	FLidarPointCloudPoint* GetData() const;

	/** Returns a pointer to the point data and prevents it from being released */
	FLidarPointCloudPoint* GetPersistentData() const;

	/** Returns a pointer to the point data */
	inline TSharedPtr<FLidarPointCloudRenderBuffer> GetDataCache() { return DataCache; }

	/** Return a pointer to the vertex factory containing pre-cached geometry */
	inline TSharedPtr<FLidarPointCloudVertexFactory> GetVertexFactory() { return VertexFactory; }
	
	/** Return a pointer to the ray tracing geometry */
	inline TSharedPtr<FLidarPointCloudRayTracingGeometry> GetRayTracingGeometry() { return RayTracingGeometry; }

	/**
	 * Builds and updates the necessary render data buffers
	 * Returns true if successful
	 */
	bool BuildDataCache(bool bUseStaticBuffers, bool bUseRayTracing);

	/** Returns the sum of grid and padding points allocated to this node. */
	inline uint32 GetNumPoints() const { return NumPoints; }

	/** Returns the sum of visible grid and padding points allocated to this node. */
	uint32 GetNumVisiblePoints() const { return NumVisiblePoints; }

	/** Calculates and returns the bounds of this node */
	inline FBox GetBounds() const;

	/** Calculates and returns the sphere bounds of this node */
	inline FSphere GetSphereBounds() const;

	/** Returns a pointer to the node at the given location, or null if one doesn't exist yet. */
	FLidarPointCloudOctreeNode* GetChildNodeAtLocation(const uint8& Location) const;

	uint8 GetChildrenBitmask() const;

	void UpdateNumVisiblePoints();

	/** Attempts to insert given points to this node or passes it to the children, otherwise. */
	void InsertPoints(const FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation);
	void InsertPoints(FLidarPointCloudPoint** Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation);

	/** Removes all points. */
	void Empty(bool bRecursive = true);

	/** Returns the maximum depth of any children of this node .*/
	uint32 GetMaxDepth() const;

	/** Returns the amount of memory used by this node */
	int64 GetAllocatedSize(bool bRecursive, bool bIncludeBulkData) const;

	/** Returns true, if the node has its data loaded */
	bool HasData() const { return bHasData; }

	/** Returns true, if the node has its data loaded */
	bool HasRenderData() const { return DataCache.IsValid() || VertexFactory.IsValid(); }

	/**
	 * Releases the BulkData
	 * If forced, the node will be released even if persistent
	 */
	void ReleaseData(bool bForce = false);
	
	/** Releases and removes the render data cache */
	void ReleaseDataCache();

	/** Convenience function, to add point statistics to the Tree table. */
	void AddPointCount(int32 PointCount);

	/** Sorts the points by visibility (visible first) to optimize data processing and rendering */
	void SortVisiblePoints();

private:
	template <typename T>
	void InsertPoints_Internal(T Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation);
	void InsertPoints_Dynamic(const FLidarPointCloudPoint* Points, const int64& Count, const FVector3f& Translation);
	void InsertPoints_Static(const FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation);
	void InsertPoints_Dynamic(FLidarPointCloudPoint** Points, const int64& Count, const FVector3f& Translation);
	void InsertPoints_Static(FLidarPointCloudPoint** Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, const FVector3f& Translation);

	friend FLidarPointCloudOctree;
	friend FLidarPointCloudTraversalOctree;
	friend FLidarPointCloudTraversalOctreeNode;
	friend void LidarPointCloudMeshing::CalculateNormals(FLidarPointCloudOctree*, FThreadSafeBool*, int32, float, TArray64<FLidarPointCloudPoint *>&);
};

/**
 * Used for efficient handling of point cloud data.
 */
class LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudOctree
{
public:
	/** Stores shared per-LOD node data. */
	struct FSharedLODData
	{
		float Radius;
		float RadiusSq;
		float GridSize;
		FVector3f GridSize3D;
		float Size;
		float NormalizationMultiplier;
		FVector3f Extent;

		FSharedLODData() {}
		FSharedLODData(const FVector3f& InExtent);
	};

public:
	/** Maximum allowed depth for any node */
	static int32 MaxNodeDepth;

	/** Maximum number of unallocated points to keep inside the node before they need to be converted in to a full child node */
	static int32 MaxBucketSize;

	/** Virtual grid resolution to divide the node into */
	static int32 NodeGridResolution;

	/** Used for thread safety between rendering and asset operations. */
	mutable FCriticalSection DataLock;

	/** Used to prevent auto-release of nodes if they are in use by other threads */
	FCriticalSection DataReleaseLock;

private:
	FLidarPointCloudOctreeNode* Root;
	
	/** Stores shared per-LOD node data. */
	TArray<FSharedLODData> SharedData;
	
	/** Stores number of points per each LOD. */
	TArray<FThreadSafeCounter64> PointCount;

	/** Stores number of nodes per each LOD. */
	TArray<FThreadSafeCounter> NodeCount;

	/** Extent of this Cloud. */
	FVector3f Extent;

	/** Used to cache the Allocated Size. */
	mutable int32 PreviousNodeCount;
	mutable int64 PreviousPointCount;
	mutable int64 PreviousAllocatedStructureSize;
	mutable int64 PreviousAllocatedSize;

	/** Used to notify any linked traversal octrees when they need to re-generate the data. */
	TArray<TWeakPtr<FLidarPointCloudTraversalOctree, ESPMode::ThreadSafe>> LinkedTraversalOctrees;

	/** Stores collision mesh data */
	FTriMeshCollisionData CollisionMesh;

	/** Pointer to the owner of this Octree */
	ULidarPointCloud* Owner;

	IAsyncReadFileHandle* ReadHandle;

	FByteBulkData BulkData;

#if WITH_EDITOR
	struct FLidarPointCloudBulkData : public FBulkData
	{
		FSerializeBulkDataElements SerializeElementsCallback;
		
		FLidarPointCloudBulkData(FLidarPointCloudOctree* Octree);

		void Serialize(FArchive& Ar, UObject* InOwner);
	} SavingBulkData;
#endif
	
	TQueue<FLidarPointCloudOctreeNode*> QueuedNodes;
	TArray<FLidarPointCloudOctreeNode*> NodesInUse;

	TAtomic<bool> bStreamingBusy;

	/** Set to true when the Octree is persistently force-loaded. */
	bool bIsFullyLoaded;

public:
	FLidarPointCloudOctree() : FLidarPointCloudOctree(nullptr) {}
	FLidarPointCloudOctree(ULidarPointCloud* Owner);
	~FLidarPointCloudOctree();
	FLidarPointCloudOctree(const FLidarPointCloudOctree&) = delete;
	FLidarPointCloudOctree(FLidarPointCloudOctree&&) = delete;
	FLidarPointCloudOctree& operator=(const FLidarPointCloudOctree&) = delete;
	FLidarPointCloudOctree& operator=(FLidarPointCloudOctree&&) = delete;

	/** Returns true if the Root node exists and has any data assigned. */
	bool HasData() const { return Root->GetNumPoints() > 0; }

	/** Returns the number of different LODs. */
	int32 GetNumLODs() const;

	/** Returns the Cloud bounds. */
	FBox GetBounds() const { return FBox(-Extent, Extent); }

	/** Returns the extent of the Cloud's bounds. */
	inline FVector3f GetExtent() const { return Extent; }

	/** Recalculates and updates points bounds. */
	void RefreshBounds();

	/** Returns the total number of points. */
	int64 GetNumPoints() const;

	/** Returns the total number of visible points. */
	int64 GetNumVisiblePoints() const;

	/** Returns the total number of nodes. */
	int32 GetNumNodes() const;

	/** Returns the total number of nodes. */
	inline int32 GetNumNodesInUse() const { return NodesInUse.Num(); }

	/** Returns a pointer to the Point Cloud asset, which owns this Octree. */
	ULidarPointCloud* GetOwner() const { return Owner; }

	/** Returns the amount of memory used by this Octree, including the BulkData */
	int64 GetAllocatedSize() const;

	/** Returns the amount of memory used by this Octree, excluding the BulkData */
	int64 GetAllocatedStructureSize() const;

	/** Returns the grid cell size at root level. */
	float GetRootCellSize() const { return SharedData[0].GridSize; }

	/** Returns an estimated spacing between points */
	float GetEstimatedPointSpacing() const;

	/** Returns true, if the Octree has collision built */
	bool HasCollisionData() const { return CollisionMesh.Vertices.Num() > 0; }

	/** Builds collision using the accuracy provided */
	void BuildCollision(const float& Accuracy, const bool& bVisibleOnly);

	/** Removes collision mesh data */
	void RemoveCollision();

	/** Returns pointer to the collision data */
	const FTriMeshCollisionData* GetCollisionData() const { return &CollisionMesh; }

	/** Constructs and returns the MeshBuffers struct from the data */
	void BuildStaticMeshBuffers(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform);

	/** Populates the given array with points from the tree */
	void GetPoints(TArray<FLidarPointCloudPoint*>& SelectedPoints, int64 StartIndex = 0, int64 Count = -1);
	void GetPoints(TArray64<FLidarPointCloudPoint*>& SelectedPoints, int64 StartIndex = 0, int64 Count = -1);

	/** Populates the array with the list of points within the given sphere. */
	void GetPointsInSphere(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly);
	void GetPointsInSphere(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly);

	/** Populates the array with the list of pointers to points within the given box. */
	void GetPointsInBox(TArray<const FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly) const;
	void GetPointsInBox(TArray64<const FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly) const;
	void GetPointsInBox(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly);
	void GetPointsInBox(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly);

	/** Populates the array with the list of points within the given convex volume. */
	void GetPointsInConvexVolume(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly);
	void GetPointsInConvexVolume(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly);

	/** Populates the given array with copies of points from the tree */
	void GetPointsAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FTransform* LocalToWorld, int64 StartIndex = 0, int64 Count = -1) const;
	void GetPointsAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FTransform* LocalToWorld, int64 StartIndex = 0, int64 Count = -1) const;

	/** Executes the provided action on batches of points. */
	void GetPointsAsCopiesInBatches(TFunction<void(TSharedPtr<TArray64<FLidarPointCloudPoint>>)> Action, const int64& BatchSize, const bool& bVisibleOnly);

	/** Populates the array with the list of points within the given sphere. */
	void GetPointsInSphereAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const FTransform* LocalToWorld) const;
	void GetPointsInSphereAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const FTransform* LocalToWorld) const;

	/** Populates the array with the list of pointers to points within the given box. */
	void GetPointsInBoxAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const FTransform* LocalToWorld) const;
	void GetPointsInBoxAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const FTransform* LocalToWorld) const;

	/** Performs a raycast test against the point cloud. Returns the pointer if hit or nullptr otherwise. */
	FLidarPointCloudPoint* RaycastSingle(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly);

	/**
	 * Performs a raycast test against the point cloud.
	 * Populates OutHits array with the results.
	 * Returns true it anything has been hit.
	 */
	bool RaycastMulti(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly, TArray<FLidarPointCloudPoint*>& OutHits);
	bool RaycastMulti(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly, const FTransform* LocalToWorld, TArray<FLidarPointCloudPoint>& OutHits);

	/** Returns true if there are any points within the given sphere. */
	bool HasPointsInSphere(const FSphere& Sphere, const bool& bVisibleOnly) const;

	/** Returns true if there are any points within the given box. */
	bool HasPointsInBox(const FBox& Box, const bool& bVisibleOnly) const;

	/** Returns true if there are any points hit by the given ray. */
	bool HasPointsByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly) const;

	/** Sets visibility of points within the given sphere. */
	void SetVisibilityOfPointsInSphere(const bool& bNewVisibility, const FSphere& Sphere);

	/** Sets visibility of points within the given box. */
	void SetVisibilityOfPointsInBox(const bool& bNewVisibility, const FBox& Box);

	/** Sets visibility of the first point hit by the given ray. */
	void SetVisibilityOfFirstPointByRay(const bool& bNewVisibility, const FLidarPointCloudRay& Ray, const float& Radius);

	/** Sets visibility of points hit by the given ray. */
	void SetVisibilityOfPointsByRay(const bool& bNewVisibility, const FLidarPointCloudRay& Ray, const float& Radius);

	/** Marks all points hidden */
	void HideAll();

	/** Marks all points visible */
	void UnhideAll();

	/** Executes the provided action on each of the points. */
	void ExecuteActionOnAllPoints(TFunction<void(FLidarPointCloudPoint*)> Action, const bool& bVisibleOnly);

	/** Executes the provided action on each of the points within the given sphere. */
	void ExecuteActionOnPointsInSphere(TFunction<void(FLidarPointCloudPoint*)> Action, const FSphere& Sphere, const bool& bVisibleOnly);

	/** Executes the provided action on each of the points within the given box. */
	void ExecuteActionOnPointsInBox(TFunction<void(FLidarPointCloudPoint*)> Action, const FBox& Box, const bool& bVisibleOnly);

	/** Executes the provided action on the first point hit by the given ray. */
	void ExecuteActionOnFirstPointByRay(TFunction<void(FLidarPointCloudPoint*)> Action, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly);

	/** Executes the provided action on each of the points hit by the given ray. */
	void ExecuteActionOnPointsByRay(TFunction<void(FLidarPointCloudPoint*)> Action, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly);

	/** Applies the given color to all points */
	void ApplyColorToAllPoints(const FColor& NewColor, const bool& bVisibleOnly);

	/** Applies the given color to all points within the sphere */
	void ApplyColorToPointsInSphere(const FColor& NewColor, const FSphere& Sphere, const bool& bVisibleOnly);

	/** Applies the given color to all points within the box */
	void ApplyColorToPointsInBox(const FColor& NewColor, const FBox& Box, const bool& bVisibleOnly);

	/** Applies the given color to the first point hit by the given ray */
	void ApplyColorToFirstPointByRay(const FColor& NewColor, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly);

	/** Applies the given color to all points hit by the given ray */
	void ApplyColorToPointsByRay(const FColor& NewColor, const FLidarPointCloudRay& Ray, const float& Radius, bool bVisibleOnly);

	/**
	 * This should to be called if any manual modification to individual points' visibility has been made.
	 * If not marked dirty, the rendering may work suboptimally.
	 */
	void MarkPointVisibilityDirty();

	/** Marks render data of all nodes as dirty. */
	void MarkRenderDataDirty();

	/** Marks render data of all nodes within the given sphere as dirty. */
	void MarkRenderDataInSphereDirty(const FSphere& Sphere);
	
	/** Marks render data of all nodes within the given convex volume as dirty. */
	void MarkRenderDataInConvexVolumeDirty(const FConvexVolume& ConvexVolume);

#if WITH_EDITOR
	void SelectByConvexVolume(const FConvexVolume& ConvexVolume, bool bAdditive, bool bVisibleOnly);
	void SelectBySphere(const FSphere& Sphere, bool bAdditive, bool bVisibleOnly);
	void HideSelected();
	void DeleteSelected();
	void InvertSelection();
	int64 NumSelectedPoints() const;
	bool HasSelectedPoints() const;
	void GetSelectedPoints(TArray64<FLidarPointCloudPoint*>& SelectedPoints) const;
	template <typename T>
	void GetSelectedPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FTransform& Transform) const;
	void GetSelectedPointsAsCopies(TArray<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const;	
	void GetSelectedPointsAsCopies(TArray64<FLidarPointCloudPoint>& SelectedPoints, const FTransform& Transform) const;	
	void GetSelectedPointsInBox(TArray64<const FLidarPointCloudPoint*>& SelectedPoints, const FBox& Box) const;
	void ClearSelection();
	void BuildStaticMeshBuffersForSelection(float CellSize, LidarPointCloudMeshing::FMeshBuffers* OutMeshBuffers, const FTransform& Transform);
#endif

	/** Initializes the Octree properties. */
	void Initialize(const FVector3f& InExtent);

	/** Inserts the given point into the Octree structure, internally thread-safe. */
	void InsertPoint(const FLidarPointCloudPoint* Point, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation);

	/** Inserts group of points into the Octree structure, internally thread-safe. */
	void InsertPoints(FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation);
	void InsertPoints(const FLidarPointCloudPoint* Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation);
	void InsertPoints(FLidarPointCloudPoint** Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation);

	/** Attempts to remove the given point.  */
	void RemovePoint(const FLidarPointCloudPoint* Point);
	void RemovePoint(FLidarPointCloudPoint Point);

	/** Removes points in bulk */
	void RemovePoints(TArray<FLidarPointCloudPoint*>& Points);
	void RemovePoints(TArray64<FLidarPointCloudPoint*>& Points);

	/** Removes all points within the given sphere */
	void RemovePointsInSphere(const FSphere& Sphere, const bool& bVisibleOnly);

	/** Removes all points within the given box */
	void RemovePointsInBox(const FBox& Box, const bool& bVisibleOnly);

	/** Removes the first point hit by the given ray */
	void RemoveFirstPointByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly);

	/** Removes all points hit by the given ray */
	void RemovePointsByRay(const FLidarPointCloudRay& Ray, const float& Radius, const bool& bVisibleOnly);

	/** Removes all hidden points */
	void RemoveHiddenPoints();

	/** Resets all normals information */
	void ResetNormals();

	/**
	 * Calculates Normals for the provided points
	 * If a nullptr is passed, the calculation will be executed on the whole cloud
	 */
	void CalculateNormals(FThreadSafeBool* bCancelled, int32 Quality, float Tolerance, TArray64<FLidarPointCloudPoint*>* InPointSelection);

	/** Removes all points and, optionally, all nodes except for the root node. Retains the bounds. */
	void Empty(bool bDestroyNodes);

	/** Adds the given traversal octree to the list of linked octrees. */
	void RegisterTraversalOctree(TWeakPtr<FLidarPointCloudTraversalOctree, ESPMode::ThreadSafe> TraversalOctree)
	{
		if (TraversalOctree.IsValid())
		{
			LinkedTraversalOctrees.Add(TraversalOctree);
		}
	}

	/** Removes the given traversal octree from the list */
	void UnregisterTraversalOctree(FLidarPointCloudTraversalOctree* TraversalOctree);

	/**
	 * Streams requested nodes or extends their lifetime, if already loaded
	 * Unloads all unused nodes with expired lifetime
	 */
	void StreamNodes(TArray<FLidarPointCloudOctreeNode*>& Nodes, const float& CurrentTime);

	/** Returns true, if the cloud is fully and persistently loaded. */
	bool IsFullyLoaded() const { return bIsFullyLoaded; }

	/** Loads all nodes. */
	void LoadAllNodes(bool bLoadPersistently);

	/**
	 * Releases all nodes.
	 * Optionally, releases persistent nodes too.
	 */
	void ReleaseAllNodes(bool bIncludePersistent);

	bool IsOptimizedForDynamicData() const;

	void OptimizeForDynamicData();

	void OptimizeForStaticData();

	IAsyncReadFileHandle* GetReadHandle();
	void CloseReadHandle();

	//~ Begin Deprecated
	UE_DEPRECATED(4.27, "Use GetPointsInConvexVolume instead.")
	void GetPointsInFrustum(TArray<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly);
	UE_DEPRECATED(4.27, "Use GetPointsInConvexVolume instead.")
	void GetPointsInFrustum(TArray64<FLidarPointCloudPoint*>& SelectedPoints, const FConvexVolume& Frustum, const bool& bVisibleOnly);
	UE_DEPRECATED(4.27, "Use MarkRenderDataInConvexVolumeDirty instead.")
	void MarkRenderDataInFrustumDirty(const FConvexVolume& Frustum);
	//~ End Deprecated

private:
	void RefreshAllocatedSize();

	template <typename T>
	void InsertPoints_Internal(T Points, const int64& Count, ELidarPointCloudDuplicateHandling DuplicateHandling, bool bRefreshPointsBounds, const FVector3f& Translation);

	template <typename T>
	void GetPoints_Internal(TArray<FLidarPointCloudPoint*, T>& Points, int64 StartIndex = 0, int64 Count = -1);
	template <typename T>
	void GetPointsInSphere_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly);
	template <typename T>
	void GetPointsInBox_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly);
	template <typename T>
	void GetPointsInBox_Internal(TArray<const FLidarPointCloudPoint*, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly) const;
	template <typename T>
	void GetPointsInConvexVolume_Internal(TArray<FLidarPointCloudPoint*, T>& SelectedPoints, const FConvexVolume& ConvexVolume, const bool& bVisibleOnly);
	template <typename T>
	void GetPointsAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& Points, const FTransform* LocalToWorld, int64 StartIndex = 0, int64 Count = -1) const;
	template <typename T>
	void GetPointsInSphereAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FSphere& Sphere, const bool& bVisibleOnly, const FTransform* LocalToWorld) const;
	template <typename T>
	void GetPointsInBoxAsCopies_Internal(TArray<FLidarPointCloudPoint, T>& SelectedPoints, const FBox& Box, const bool& bVisibleOnly, const FTransform* LocalToWorld) const;

	template <typename T>
	void RemovePoints_Internal(TArray<FLidarPointCloudPoint*, T>& Points);

	void RemovePoint_Internal(FLidarPointCloudOctreeNode* Node, int32 Index);

	/** Notifies all linked traversal octrees that they should invalidate and regenerate the data. */
	void MarkTraversalOctreesForInvalidation();

	void Serialize(FArchive& Ar);
	void SerializeBulkData(FArchive& Ar);

	void StreamNodeData(FLidarPointCloudOctreeNode* Node);

	friend FArchive& operator<<(FArchive& Ar, FLidarPointCloudOctree& O)
	{
		O.Serialize(Ar);
		return Ar;
	}

	friend FLidarPointCloudOctreeNode;
	friend FLidarPointCloudTraversalOctree;
	friend void LidarPointCloudMeshing::CalculateNormals(FLidarPointCloudOctree*, FThreadSafeBool*, int32, float, TArray64<FLidarPointCloudPoint*>&);
};

/**
 * Represents a single octant in the traversal tree.
 */
struct FLidarPointCloudTraversalOctreeNode
{
	/** Pointer to the target node. */
	FLidarPointCloudOctreeNode* DataNode;

	/** Stores the center of the target node in World space. */
	FVector3f Center;

	/** Depth of this node */
	uint8 Depth;

	/** Calculated for use with adaptive sprite scaling */
	uint8 VirtualDepth;

	FLidarPointCloudTraversalOctreeNode* Parent;

	FLidarPointCloudTraversalOctree* Octree;

	/** Stores the children array */
	TArray<FLidarPointCloudTraversalOctreeNode> Children;

	/** Holds true if the node has been selected for rendering. */
	bool bSelected;

	bool bFullyContained;

	FLidarPointCloudTraversalOctreeNode();

	/** Builds the traversal version of the given node. */
	void Build(FLidarPointCloudTraversalOctree* TraversalOctree, FLidarPointCloudOctreeNode* Node, const FTransform& LocalToWorld, const FVector3f& LocationOffset);

	/** Calculates virtual depth of this node, to be used to estimate the best sprite size */
	void CalculateVirtualDepth(const TArray<float>& LevelWeights, const float& PointSizeBias);

	inline bool IsAvailable() const { return bSelected && DataNode->HasRenderData(); }
};

/** Used for node size sorting and node selection. */
struct FLidarPointCloudTraversalOctreeNodeSizeData
{
	FLidarPointCloudTraversalOctreeNode* Node;
	float Size;
	int32 ProxyIndex;

	FLidarPointCloudTraversalOctreeNodeSizeData(FLidarPointCloudTraversalOctreeNode* Node, const float& Size, const int32& ProxyIndex);
};

/** Convenience struct to group all selection params into one */
struct FLidarPointCloudNodeSelectionParams
{
	float MinScreenSize;
	float ScreenCenterImportance;
	int32 MinDepth;
	int32 MaxDepth;
	float BoundsScale;
	bool bUseFrustumCulling;
	const TArray<struct FLidarPointCloudClippingVolumeParams>* ClippingVolumes;
};

/**
 * Used as a traversal tree for node selection during rendering
 */
struct FLidarPointCloudTraversalOctree
{
	FLidarPointCloudTraversalOctreeNode Root;

	/** Stores per-LOD bounds in World space. */
	TArray<float> RadiiSq;
	TArray<FVector3f> Extents;

	/** Stores the number of LODs. */
	uint8 NumLODs;

	/** Normalized histogram of level weights, one for each LOD. Used for point scaling */
	TArray<float> LevelWeights;

	float VirtualDepthMultiplier;
	float ReversedVirtualDepthMultiplier;

	/** Pointer to the source Octree */
	FLidarPointCloudOctree* Octree;

	bool bValid;

	/** Build the Traversal tree from the Octree provided */
	FLidarPointCloudTraversalOctree(FLidarPointCloudOctree* Octree, const FTransform& LocalToWorld);

	~FLidarPointCloudTraversalOctree();
	FLidarPointCloudTraversalOctree(const FLidarPointCloudTraversalOctree&) = delete;
	FLidarPointCloudTraversalOctree(FLidarPointCloudTraversalOctree&&) = delete;
	FLidarPointCloudTraversalOctree& operator=(const FLidarPointCloudTraversalOctree&) = delete;
	FLidarPointCloudTraversalOctree& operator=(FLidarPointCloudTraversalOctree&&) = delete;

	/**
	 * Selects and appends the subset of visible nodes for rendering.
	 * Returns number of selected nodes
	 */
	int32 GetVisibleNodes(TArray<FLidarPointCloudTraversalOctreeNodeSizeData>& NodeSizeData, const struct FLidarPointCloudViewData* ViewData, const int32& ProxyIndex, const FLidarPointCloudNodeSelectionParams& SelectionParams);

	void CalculateVisibilityStructure(TArray<uint32>& OutData);

	void CalculateLevelWeightsForSelectedNodes(TArray<float>& OutLevelWeights);

	FVector GetCenter() const { return (FVector)Root.Center; }
	FVector GetExtent() const { return (FVector)Extents[0]; }
};
