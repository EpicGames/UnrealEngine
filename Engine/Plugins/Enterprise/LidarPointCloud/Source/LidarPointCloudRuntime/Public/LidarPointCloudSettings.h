// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "LidarPointCloudSettings.generated.h"

UENUM(BlueprintType)
enum class ELidarPointCloudDuplicateHandling : uint8
{
	/** Keeps any duplicates found */
	Ignore,
	/** Keeps the first point and skips any further duplicates */
	SelectFirst,
	/** Selects the brightest of the duplicates */
	SelectBrighter
};

UCLASS(config=Engine, defaultconfig)
class LIDARPOINTCLOUDRUNTIME_API ULidarPointCloudSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Determines how to handle duplicate points (distance < 0.0001). */
	UPROPERTY(config, EditAnywhere, Category=Octree)
	ELidarPointCloudDuplicateHandling DuplicateHandling;

	/** Maximum distance between points, within which they are considered to be duplicates */
	UPROPERTY(config, EditAnywhere, Category=Octree, meta = (ClampMin = "0"))
	float MaxDistanceForDuplicate;

	/**
	 * Maximum number of unallocated points to keep inside the node before they need to be converted in to a full child node.
	 * Lower values will provide finer LOD control at the expense of system RAM and CPU time.
	 */
	UPROPERTY(config, EditAnywhere, Category=Octree)
	int32 MaxBucketSize;

	/**
	 * Virtual grid resolution to divide the node into.
	 * Lower values will provide finer LOD control at the expense of system RAM and CPU time.
	 */
	UPROPERTY(config, EditAnywhere, Category=Octree)
	int32 NodeGridResolution;

	/** Determines the maximum amount of points to process in a single batch when using multi-threading. */
	UPROPERTY(config, EditAnywhere, Category=Performance)
	int32 MultithreadingInsertionBatchSize;

	/** Enabling this will allow editor to import the point clouds in the background, without blocking the main thread. */
	UPROPERTY(config, EditAnywhere, Category=Performance)
	bool bUseAsyncImport;

	/**
	 * Enabling this will allocate larger portion of the available point budget to the viewport with focus.
	 * May improve asset editing experience, if the scenes are busy.
	 * Disable, if you are experiencing visual glitches.
	 */
	UPROPERTY(config, EditAnywhere, Category=Performance)
	bool bPrioritizeActiveViewport;

	/**
	 * Sets how long the nodes wil be kept in RAM after they are no longer visible.
	 * Larger values are more likely to avoid re-loads from storage, at the cost of increased RAM usage. 
	 */
	UPROPERTY(config, EditAnywhere, Category=Performance, meta = (ClampMin = "0"))
	float CachedNodeLifetime;

	/**
     * Enabling this will automatically release memory used by the asset once it's saved
     * Helpful when dealing with very large data sets to avoid memory blocking
     */
    UPROPERTY(config, EditAnywhere, Category= Performance)
    bool bReleaseAssetAfterSaving;

    /**
     * Enabling this will automatically release memory used by the asset once it's cooked
     * Helpful when dealing with very large data sets to avoid memory blocking
     */
    UPROPERTY(config, EditAnywhere, Category= Performance)
    bool bReleaseAssetAfterCooking;

	/** If enabled, the render data generation will be spread across multiple frames to avoid freezes */
	UPROPERTY(config, EditAnywhere, Category=Performance)
	bool bUseRenderDataSmoothing;

	/** If UseRenderDataSmoothing is enabled, this will determine how much of the frame time can be spent on render data generation. */
	UPROPERTY(config, EditAnywhere, Category=Performance, meta = (EditCondition = "bUseAsyncImport"))
	float RenderDataSmoothingMaxFrametime;

	/** Enabling this will greatly improve runtime performance at a cost of quadrupling VRAM use */
	UPROPERTY(config, EditAnywhere, Category=Performance)
	bool bUseFastRendering;

	/**
	 * If enabled, Lidar assets will become visible to Ray Tracing.
	 * Warning, this will significantly increase VRAM usage!
	 */
	UPROPERTY(config, EditAnywhere, Category=RayTracing)
	bool bEnableLidarRayTracing;
	
	/** Affects the size of per-thread data for the meshing algorithm. */
	UPROPERTY(config, EditAnywhere, Category=Collision)
	int32 MeshingBatchSize;

	/**
	 * Automatically centers the cloud on import.
	 * Caution: Preserving original coordinates may cause noticeable precision loss, if the values are too large.
	 * Should you experience point 'banding' effect, please re-import your cloud with centering enabled.
	 */
	UPROPERTY(config, EditAnywhere, Category="Automation")
	bool bAutoCenterOnImport;

	/** If enabled, the assets will automatically calculate normals upon their successful import. */
	UPROPERTY(config, EditAnywhere, Category="Automation")
	bool bAutoCalculateNormalsOnImport;

	/** If enabled, the assets will automatically build collision upon their successful import. */
	UPROPERTY(config, EditAnywhere, Category="Automation")
	bool bAutoBuildCollisionOnImport;

	/** Scale to apply during import */
	UPROPERTY(config, EditAnywhere, Category= "Import / Export", meta = (ClampMin = "0.0001"))
	float ImportScale;

	/** Scale to apply during export. In most cases, this should be equal to an inverted ImportScale */
	UPROPERTY(config, EditAnywhere, Category= "Import / Export", meta = (ClampMin = "0.0001"))
	float ExportScale;

public:
	ULidarPointCloudSettings();
};