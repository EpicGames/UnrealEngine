// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLidarPointCloud, Log, All);
DECLARE_STATS_GROUP(TEXT("Lidar Point Cloud"), STATGROUP_LidarPointCloud, STATCAT_Advanced);

#define PC_LOG(Format, ...) UE_LOG(LogLidarPointCloud, Log, TEXT(Format), ##__VA_ARGS__)
#define PC_WARNING(Format, ...) UE_LOG(LogLidarPointCloud, Warning, TEXT(Format), ##__VA_ARGS__)
#define PC_ERROR(Format, ...) UE_LOG(LogLidarPointCloud, Error, TEXT(Format), ##__VA_ARGS__)

#pragma pack(push)
#pragma pack(1)
/** 3D vector represented using only a single byte per component */
USTRUCT(BlueprintType)
struct LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudNormal
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Normal")
	uint8 X;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Normal")
	uint8 Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Normal")
	uint8 Z;

public:
	FLidarPointCloudNormal() { Reset(); }
	FLidarPointCloudNormal(const FVector3f& Normal) { SetFromVector(Normal); }
	FLidarPointCloudNormal(const FPlane& Normal) { SetFromFloats(Normal.X, Normal.Y, Normal.Z); }
	FLidarPointCloudNormal(const float& X, const float& Y, const float& Z) { SetFromFloats(X, Y, Z); }

	bool operator==(const FLidarPointCloudNormal& Other) const { return X == Other.X && Y == Other.Y && Z == Other.Z; }

	inline bool IsValid() const { return X != 127 || Y != 127 || Z != 127; }

	inline void SetFromVector(const FVector3f& Normal)
	{
		SetFromFloats(Normal.X, Normal.Y, Normal.Z);
	}
	inline void SetFromFloats(const float& InX, const float& InY, const float& InZ)
	{
		X = FMath::Min((InX + 1) * 127.5f, 255.0f);
		Y = FMath::Min((InY + 1) * 127.5f, 255.0f);
		Z = FMath::Min((InZ + 1) * 127.5f, 255.0f);
	}

	inline void Reset()
	{
		X = Y = Z = 127;
	}

	inline FVector3f ToVector() const { return FVector3f(X / 127.5f - 1, Y / 127.5f - 1, Z / 127.5f - 1); }
};

/** Used for backwards compatibility with pre-normal datasets */
struct FLidarPointCloudPoint_Legacy
{
	FVector3f Location;
	FColor Color;
	uint8 bVisible : 1;
	uint8 ClassificationID : 5;
	uint8 Dummy : 2;
};

USTRUCT(BlueprintType)
struct LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudPoint
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Cloud Point")
	FVector3f Location;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Cloud Point")
	FColor Color;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Cloud Point")
	FLidarPointCloudNormal Normal;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lidar Point Cloud Point")
	uint8 bVisible : 1;

	/** Valid range is 0 - 31. */
	uint8 ClassificationID : 5;

	uint8 bSelected : 1;

private:	
	uint8 bMarkedForDeletion : 1;

public:
	FLidarPointCloudPoint()
		: Location(FVector3f::ZeroVector)
		, Color(FColor::White)
		, Normal()
		, bVisible(true)
		, ClassificationID(0)
		, bSelected(false)
		, bMarkedForDeletion(false)
	{
	}
	FLidarPointCloudPoint(const float& X, const float& Y, const float& Z)
		: FLidarPointCloudPoint()
	{
		Location.X = X;
		Location.Y = Y;
		Location.Z = Z;
	}
	FLidarPointCloudPoint(const float& X, const float& Y, const float& Z, const float& I)
		: FLidarPointCloudPoint(X, Y, Z)
	{
		Color.A = FMath::RoundToInt(FMath::Clamp(I, 0.0f, 1.0f) * 255.f);
	}
	FLidarPointCloudPoint(const float& X, const float& Y, const float& Z, const float& R, const float& G, const float& B, const float& A = 1.0f)
		: FLidarPointCloudPoint(X, Y, Z)
	{
		Color = FLinearColor(R, G, B, A).ToFColor(false);
	}
	FLidarPointCloudPoint(const float& X, const float& Y, const float& Z, const float& R, const float& G, const float& B, const float& A, const float& NX, const float& NY, const float& NZ)
		: FLidarPointCloudPoint(X, Y, Z)
	{
		Color = FLinearColor(R, G, B, A).ToFColor(false);
		Normal.SetFromFloats(NX, NY, NZ);
	}
	FLidarPointCloudPoint(const FVector3f& Location) : FLidarPointCloudPoint(Location.X, Location.Y, Location.Z) {}
	FLidarPointCloudPoint(const FVector3f& Location, const float& R, const float& G, const float& B, const float& A = 1.0f)
		: FLidarPointCloudPoint(Location)
	{
		Color = FLinearColor(R, G, B, A).ToFColor(false);
	}
	FLidarPointCloudPoint(const FVector3f& Location, const float& R, const float& G, const float& B, const float& A, const uint8& ClassificationID)
		: FLidarPointCloudPoint(Location, R, G, B, A)
	{
		this->ClassificationID = ClassificationID;
	}
	FLidarPointCloudPoint(const FVector3f& Location, const uint8& R, const uint8& G, const uint8& B, const uint8& A, const uint8& ClassificationID)
		: FLidarPointCloudPoint(Location.X, Location.Y, Location.Z)
	{
		Color = FColor(R, G, B, A);
		this->ClassificationID = ClassificationID;
	}
	FLidarPointCloudPoint(const FVector3f& Location, const FColor& Color, const bool& bVisible, const uint8& ClassificationID)
		: FLidarPointCloudPoint(Location)
	{
		this->Color = Color;
		this->bVisible = bVisible;
		this->ClassificationID = ClassificationID;
	}
	FLidarPointCloudPoint(const FVector3f& Location, const FColor& Color, const bool& bVisible, const uint8& ClassificationID, const FLidarPointCloudNormal& Normal)
		: FLidarPointCloudPoint(Location)
	{
		this->Color = Color;
		this->bVisible = bVisible;
		this->ClassificationID = ClassificationID;
		this->Normal = Normal;
	}
	// Questionable behavior here - the copy constructor only copies part of the state but the assignment operator copies it all
	FLidarPointCloudPoint(const FLidarPointCloudPoint& Other)
		: FLidarPointCloudPoint()
	{
		CopyFrom(Other);
	}
	FLidarPointCloudPoint& operator=(const FLidarPointCloudPoint&) = default;
	FLidarPointCloudPoint(const FLidarPointCloudPoint_Legacy& Other)
		: FLidarPointCloudPoint(Other.Location, Other.Color, Other.bVisible, Other.ClassificationID)
	{
	}

	inline void CopyFrom(const FLidarPointCloudPoint& Other)
	{
		Location = Other.Location;
		Color = Other.Color;
		Normal = Other.Normal;
		bVisible = Other.bVisible;
		ClassificationID = Other.ClassificationID;
	}

	inline FLidarPointCloudPoint Transform(const FTransform3f& Transform) const
	{
		return FLidarPointCloudPoint(Transform.TransformPosition(Location), Color, bVisible, ClassificationID);
	}

	bool operator==(const FLidarPointCloudPoint& P) const { return Location == P.Location && Color == P.Color && bVisible == P.bVisible && ClassificationID == P.ClassificationID && Normal == P.Normal; }

	friend class FLidarPointCloudOctree;
#if WITH_EDITOR
	friend class FLidarPointCloudEditor;
#endif
};
#pragma pack(pop)

/** Used in blueprint latent function execution */
UENUM(BlueprintType)
enum class ELidarPointCloudAsyncMode : uint8
{
	Success,
	Failure,
	Progress
};

UENUM(BlueprintType)
enum class ELidarPointCloudScalingMethod : uint8
{
	/**
	 * Points are scaled based on the estimated density of their containing node.
	 * Recommended for assets with high variance of point densities, but may produce less fine detail overall.
	 * Default method in 4.25 and 4.26
	 */
	PerNode,

	/**
	 * Similar to PerNode, but the density is calculated adaptively based on the current view.
	 * Produces good amount of fine detail while being generally resistant to density variance.
	 */
	PerNodeAdaptive,

	/**
	 * Points are scaled based on their individual calculated depth.
	 * Capable of resolving the highest amount of fine detail, but is the most susceptible to 
	 * density changes across the dataset, and may result in patches of varying point sizes.
	 */
	PerPoint,

	/**
	 * Sprites will be rendered using screen-space scaling method.
	 * In that mode, Point Size property will work as Screen Percentage.
	 */
	FixedScreenSize
};

/** Used to help track multiple buffer allocations */
class LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudDataBuffer
{
public:
	FLidarPointCloudDataBuffer() : bInUse(false), PendingSize(0) {}
	~FLidarPointCloudDataBuffer() = default;
	FLidarPointCloudDataBuffer(const FLidarPointCloudDataBuffer& Other)
	{
		Data = Other.Data;
		bInUse = false;
	}
	FLidarPointCloudDataBuffer(FLidarPointCloudDataBuffer&&) = delete;
	FLidarPointCloudDataBuffer& operator=(const FLidarPointCloudDataBuffer& Other)
	{
		Data = Other.Data;
		bInUse = false;
		return *this;
	}
	FLidarPointCloudDataBuffer& operator=(FLidarPointCloudDataBuffer&&) = delete;

	inline uint8* GetData() { return Data.GetData(); }
	inline bool InUse() const { return bInUse; }
	
	/** Marks the buffer as no longer in use so it can be reassigned to another read thread. */
	void MarkAsFree();
	void Initialize(const int32& Size);
	void Resize(const int32& NewBufferSize, bool bForce = false);

private:
	TAtomic<bool> bInUse;
	TArray<uint8> Data;
	int32 PendingSize;

	friend class FLidarPointCloudDataBufferManager;
};

/** Used to help track multiple buffer allocations */
class LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudDataBufferManager
{
public:
	/** If MaxNumberOfBuffers is 0, no limit is applied */
	FLidarPointCloudDataBufferManager(const int32& BufferSize, const int32& MaxNumberOfBuffers = 0);
	~FLidarPointCloudDataBufferManager();
	FLidarPointCloudDataBufferManager(const FLidarPointCloudDataBufferManager&) = delete;
	FLidarPointCloudDataBufferManager(FLidarPointCloudDataBufferManager&&) = delete;
	FLidarPointCloudDataBufferManager& operator=(const FLidarPointCloudDataBufferManager&) = delete;
	FLidarPointCloudDataBufferManager& operator=(FLidarPointCloudDataBufferManager&&) = delete;

	FLidarPointCloudDataBuffer* GetFreeBuffer();

	void Resize(const int32& NewBufferSize);

private:
	int32 BufferSize;
	int32 MaxNumberOfBuffers;
	int32 NumBuffersCreated;
	TList<FLidarPointCloudDataBuffer> Head;
	TList<FLidarPointCloudDataBuffer>* Tail;
};

/** Used for Raycasting */
struct LIDARPOINTCLOUDRUNTIME_API FLidarPointCloudRay
{
public:
	FVector3f Origin;

private:
	FVector3f Direction;
	FVector3f InvDirection;

public:
	FLidarPointCloudRay() : FLidarPointCloudRay(FVector3f::ZeroVector, FVector3f::ForwardVector) {}
	FLidarPointCloudRay(const FVector3f& Origin, const FVector3f& Direction) : Origin(Origin)
	{
		SetDirection(Direction);
	}
	FLidarPointCloudRay(const FVector& Origin, const FVector& Direction) : FLidarPointCloudRay((FVector3f)Origin, (FVector3f)Direction) {}

	static inline FLidarPointCloudRay FromLocations(const FVector3f& Origin, const FVector3f& Destination)
	{
		return FLidarPointCloudRay(Origin, (Destination - Origin).GetSafeNormal());
	}

	FLidarPointCloudRay& TransformBy(const FTransform& Transform)
	{
		Origin = (FVector3f)Transform.TransformPosition((FVector)Origin);
		SetDirection((FVector3f)Transform.TransformVector((FVector)Direction));
		return *this;
	}
	FLidarPointCloudRay TransformBy(const FTransform& Transform) const
	{
		return FLidarPointCloudRay(Transform.TransformPosition((FVector)Origin), Transform.TransformVector((FVector)Direction));
	}
	inline FLidarPointCloudRay ShiftBy(const FVector3f& Offset) const
	{
		return FLidarPointCloudRay(Origin + Offset, Direction);
	}
	inline FLidarPointCloudRay ShiftBy(const FVector& Offset) const
	{
		return FLidarPointCloudRay(Origin + (FVector3f)Offset, Direction);
	}

	inline FVector3f GetDirection() const { return Direction; }
	inline void SetDirection(const FVector3f& NewDirection)
	{
		Direction = NewDirection;
		InvDirection = Direction.Reciprocal();
	}

	/** An Efficient and Robust Ray-Box Intersection Algorithm. Amy Williams et al. 2004. */
	inline bool Intersects(const FBox& Box) const
	{
		float tmin, tmax, tymin, tymax, tzmin, tzmax;

		tmin = ((InvDirection.X < 0 ? Box.Max.X : Box.Min.X) - Origin.X) * InvDirection.X;
		tmax = ((InvDirection.X < 0 ? Box.Min.X : Box.Max.X) - Origin.X) * InvDirection.X;
		tymin = ((InvDirection.Y < 0 ? Box.Max.Y : Box.Min.Y) - Origin.Y) * InvDirection.Y;
		tymax = ((InvDirection.Y < 0 ? Box.Min.Y : Box.Max.Y) - Origin.Y) * InvDirection.Y;

		if ((tmin > tymax) || (tymin > tmax))
		{
			return false;
		}

		if (tymin > tmin)
		{
			tmin = tymin;
		}

		if (tymax < tmax)
		{
			tmax = tymax;
		}

		tzmin = ((InvDirection.Z < 0 ? Box.Max.Z : Box.Min.Z) - Origin.Z) * InvDirection.Z;
		tzmax = ((InvDirection.Z < 0 ? Box.Min.Z : Box.Max.Z) - Origin.Z) * InvDirection.Z;

		if ((tmin > tzmax) || (tzmin > tmax))
		{
			return false;
		}

		return true;
	}
	inline bool Intersects(const FLidarPointCloudPoint* Point, const float& RadiusSq) const
	{
		const FVector3f L = Point->Location - Origin;
		const float tca = FVector3f::DotProduct(L, Direction);
		
		if (tca < 0)
		{
			return false;
		}
		
		const float d2 = FVector3f::DotProduct(L, L) - tca * tca;

		return d2 <= RadiusSq;
	}
};

UENUM(BlueprintType)
enum class ELidarClippingVolumeMode : uint8
{
	/** This will clip all points inside the volume */
	ClipInside,
	/** This will clip all points outside of the volume */
	ClipOutside,
};

/** Used to pass clipping information for async processing, to avoid accessing UObjects in non-GT */
struct FLidarPointCloudClippingVolumeParams
{
	ELidarClippingVolumeMode Mode;
	int32 Priority;
	FBox Bounds;
	FMatrix PackedShaderData;

	inline bool operator<(const FLidarPointCloudClippingVolumeParams& O) const
	{
		return (Priority < O.Priority) || (Priority == O.Priority && Mode > O.Mode);
	}

	FLidarPointCloudClippingVolumeParams(const class ALidarClippingVolume* ClippingVolume);
};

UENUM(BlueprintType)
enum class ELidarPointCloudColorationMode : uint8
{
	/** Uses color tint only */
	None,
	/** Uses imported RGB / Intensity data */
	Data,
	/** Uses imported RGB / Intensity data combined with Alpha mask from Classification Colors */
	DataWithClassificationAlpha,
	/** The cloud's color will be overridden with elevation-based color */
	Elevation,
	/** The cloud's color will be overridden with relative position-based color */
	Position,
	/** Uses Classification ID of the point along with the component's Classification Colors property to sample the color */
	Classification
};

UENUM(BlueprintType)
enum class ELidarPointCloudSpriteShape : uint8
{
	Square,
	Circle,
};

/** Convenience struct to group all component's rendering params into one */
struct FLidarPointCloudComponentRenderParams
{
	int32 MinDepth;
	int32 MaxDepth;

	float BoundsScale;
	FVector3f BoundsSize;
	FVector3f LocationOffset;
	float ComponentScale;

	float PointSize;
	float PointSizeBias;
	float GapFillingStrength;
	
	bool bOwnedByEditor;
	bool bDrawNodeBounds;
	bool bUseScreenSizeScaling;
	bool bShouldRenderFacingNormals;
	bool bUseFrustumCulling;

	ELidarPointCloudColorationMode ColorSource;
	ELidarPointCloudSpriteShape PointShape;
	ELidarPointCloudScalingMethod ScalingMethod;

	FVector4f Saturation;
	FVector4f Contrast;
	FVector4f Gamma;
	FVector4f Offset;
	FVector3f ColorTint;
	float IntensityInfluence;

	TMap<int32, FLinearColor> ClassificationColors;
	FLinearColor ElevationColorBottom;
	FLinearColor ElevationColorTop;

	class UMaterialInterface* Material = nullptr;

	void UpdateFromComponent(class ULidarPointCloudComponent* Component);
};

struct FBenchmarkTimer
{
	static void Reset()
	{
		Time = FPlatformTime::Seconds();
	}
	static double Split(uint8 Decimal = 2)
	{
		double Now = FPlatformTime::Seconds();
		double Delta = Now - Time;
		Time = Now;

		uint32 Multiplier = FMath::Pow(10.f, Decimal);

		return FMath::RoundToDouble(Delta * Multiplier * 1000) / Multiplier;
	}
	static void Log(FString Text, uint8 Decimal = 2)
	{
		const double SplitTime = Split(Decimal);
		PC_LOG("%s: %f ms", *Text, SplitTime);
	}

private:
	static double Time;
};

struct FScopeBenchmarkTimer
{
public:
	bool bActive;

private:
	double Time;
	FString Label;
	float* OutTimer;

public:
	FScopeBenchmarkTimer(const FString& Label)
		: bActive(true)
		, Time(FPlatformTime::Seconds())
		, Label(Label)
		, OutTimer(nullptr)
	{
	}
	FScopeBenchmarkTimer(float* OutTimer)
		: bActive(true)
		, Time(FPlatformTime::Seconds())
		, OutTimer(OutTimer)
	{
	}
	~FScopeBenchmarkTimer()
	{
		if (bActive)
		{
			float Delta = FMath::RoundToDouble((FPlatformTime::Seconds() - Time) * 100000) * 0.01;

			if (OutTimer)
			{
				*OutTimer += Delta;
			}
			else
			{
				PC_LOG("%s: %f ms", *Label, Delta);
			}
		}
	}
};