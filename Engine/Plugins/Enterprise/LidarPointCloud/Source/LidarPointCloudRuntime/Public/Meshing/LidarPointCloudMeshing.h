// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"
#include "HAL/ThreadSafeBool.h"

class FLidarPointCloudOctree;
struct FTriMeshCollisionData;
struct FMeshDescription;

namespace LidarPointCloudMeshing
{
	struct FVertexData
	{
		FVector3f Position;
		FVector3f Normal;
		FColor Color;

		FVertexData() {}
		FVertexData(const FVector3f& Position, const FVector3f& Normal, const FColor& Color) : Position(Position), Normal(Normal), Color(Color) {}
	};
	
	struct FMeshBuffers
	{
		TArray<uint32> Indices;
		TArray<FVertexData> Vertices;
		FBoxSphereBounds Bounds;
		
		void Init(uint32 Capacity, bool bExpandIfNotEmpty, uint32& OutIndexOffset, uint32& OutVertexOffset);
		void ExpandBounds(const FBox& NewBounds);
		void NormalizeNormals();
	};
	
	void CalculateNormals(FLidarPointCloudOctree* Octree, FThreadSafeBool* bCancelled, int32 Quality, float Tolerance, TArray64<FLidarPointCloudPoint*>& InPointSelection);
	void BuildCollisionMesh(FLidarPointCloudOctree* Octree, const float& CellSize, FTriMeshCollisionData* CollisionMesh);
	void BuildStaticMeshBuffers(FLidarPointCloudOctree* Octree, const float& CellSize, bool bUseSelection, FMeshBuffers* OutMeshBuffers, const FTransform& Transform);
};