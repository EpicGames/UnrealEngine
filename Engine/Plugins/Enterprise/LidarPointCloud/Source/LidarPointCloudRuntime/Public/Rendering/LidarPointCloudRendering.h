// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LidarPointCloudShared.h"

class ILidarPointCloudSceneProxy;

/** This allows the LOD Manager to observe the SceneProxy via weak pointer */
struct FLidarPointCloudSceneProxyWrapper
{
	ILidarPointCloudSceneProxy* Proxy;
	FLidarPointCloudSceneProxyWrapper(ILidarPointCloudSceneProxy* Proxy) : Proxy(Proxy) {}
};

struct FLidarPointCloudProxyUpdateDataNode
{
private:
	struct FLidarPointCloudOctreeNode* DataNode;
	
public:
	uint8 VirtualDepth;
	int64 NumVisiblePoints;

	/**
	 * Holds render data pointers for this node
	 * This allows us to avoid exposing potentially invalid DataNode pointer in Render Thread
	 */
	TSharedPtr<class FLidarPointCloudRenderBuffer> DataCache;
	TSharedPtr<class FLidarPointCloudVertexFactory> VertexFactory;
	TSharedPtr<class FLidarPointCloudRayTracingGeometry> RayTracingGeometry;

	FLidarPointCloudProxyUpdateDataNode() : FLidarPointCloudProxyUpdateDataNode(0, 0, nullptr) {}
	FLidarPointCloudProxyUpdateDataNode(uint8 VirtualDepth, int64 NumVisiblePoints, FLidarPointCloudOctreeNode* DataNode)
		: DataNode(DataNode)
		, VirtualDepth(VirtualDepth)
		, NumVisiblePoints(NumVisiblePoints)
		, DataCache(nullptr)
		, VertexFactory(nullptr)
		, RayTracingGeometry(nullptr)
	{
	}

	/** Passthrough method */
	bool BuildDataCache(bool bUseStaticBuffers, bool bUseRayTracing);
};

/** Used to pass data to RT to update the proxy's render data */
struct FLidarPointCloudProxyUpdateData
{
	TWeakPtr<FLidarPointCloudSceneProxyWrapper, ESPMode::ThreadSafe> SceneProxyWrapper;

	/** Number of elements within the structured buffer related to this proxy */
	int32 NumElements;

	TArray<FLidarPointCloudProxyUpdateDataNode> SelectedNodes;

	float VDMultiplier;
	float RootCellSize;
	
	bool bUseStaticBuffers;
	bool bUseRayTracing;

	TArray<uint32> TreeStructure;

#if !(UE_BUILD_SHIPPING)
	/** Stores bounds of selected nodes, used for debugging */
	TArray<FBox> Bounds;
#endif

	TArray<FLidarPointCloudClippingVolumeParams> ClippingVolumes;

	FLidarPointCloudComponentRenderParams RenderParams;

	FLidarPointCloudProxyUpdateData();
};

/** Used for communication between LOD Manager and SceneProxy */
class ILidarPointCloudSceneProxy
{
public:
	/** Updates necessary render data for the proxy. Initiated via LOD Manager's Tick */
	virtual void UpdateRenderData(const FLidarPointCloudProxyUpdateData& InRenderData) = 0;
};